{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control Protocol DMX",
	"Description": "Allows interactions between DMX and RemoteControl API.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": false,
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "RemoteControlProtocolDMX",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RemoteControlProtocolDMXEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "DMXProtocol",
			"Enabled": true
		},
		{
			"Name": "DMXEngine",
			"Enabled": true
		}
	]
}