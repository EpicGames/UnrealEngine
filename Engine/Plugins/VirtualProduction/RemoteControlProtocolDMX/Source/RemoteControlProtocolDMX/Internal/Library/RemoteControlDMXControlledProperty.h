// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXEntityFixturePatch.h"
#include "UObject/StructOnScope.h"

enum class ERemoteControlDMXPatchGroupMode : uint8;
struct FRemoteControlDMXProtocolEntity;
struct FRemoteControlProperty;
struct FRemoteControlProtocolEntity;

namespace UE::RemoteControl::DMX
{
	/** Defines a Patch in a DMX Library of a Remote Control Preset along with the properties it controls */
	class REMOTECONTROLPROTOCOLDMX_API FRemoteControlDMXControlledProperty
		: public TSharedFromThis<FRemoteControlDMXControlledProperty>
	{
	public:
		FRemoteControlDMXControlledProperty(const TSharedRef<const FRemoteControlProperty> InExposedProperty);

		/** Returns the DMX protocol entities as structs on scope */
		TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> GetEntities() const { return Entities; }

		/** Returns the owner object of this property */
		const UObject* GetOwnerActor() const;

		/** Returns the fixture patch of this property */
		UDMXEntityFixturePatch* GetFixturePatch() const;

		/** Returns the index of specified entity. Useful to enumerate attributes. The entity has to be owned by this DMX controlled property, checked. */
		int32 GetEntityIndexChecked(const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity);

		/** The DMX controlled property */
		const TSharedRef<const FRemoteControlProperty> ExposedProperty;

	private:
		/** Initializes new entities */
		void InitializeNewEntities();

		/** Makes sure all entities use unfied properties where required. */
		void UnifyEntities();

		/** Returns the subobject path for this property */
		FString GetSubobjectPath() const;

		/** The entities that control the property */
		TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> Entities;
	};
}
