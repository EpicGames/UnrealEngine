// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class UDMXLibrary;
class URemoteControlDMXUserData;
enum class ERemoteControlDMXPatchGroupMode : uint8;
struct FDMXFixtureFunction;
struct FRemoteControlProperty;
struct FRemoteControlProtocolEntity;

namespace UE::RemoteControl::DMX
{
	class FRemoteControlDMXControlledProperty;

	/** A collection of DMX controlled properties that form a DMX fixture patch together */
	class REMOTECONTROLPROTOCOLDMX_API FRemoteControlDMXControlledPropertyPatch
		: public TSharedFromThis<FRemoteControlDMXControlledPropertyPatch>
	{
	public:
		/** Constructs the DMX controlled properties patch. */
		FRemoteControlDMXControlledPropertyPatch(
			URemoteControlDMXUserData& DMXUserData, 
			const TArray<TSharedRef<FRemoteControlDMXControlledProperty>>& InDMXControlledProperties);

		~FRemoteControlDMXControlledPropertyPatch();

		/** Returns the DMX controlled properties that form this patch */
		const TArray<TSharedRef<FRemoteControlDMXControlledProperty>>& GetDMXControlledProperties() const { return DMXControlledProperties; }

		/** Returns the fixture patch corresponding to this property patch */
		UDMXEntityFixturePatch* GetFixturePatch() const;

		/** Returns the owner actor of this patch, or nullptr if there is no outer Actor. */
		const UObject* GetOwnerActor() const;

	private:
#if WITH_EDITOR
		/** Called when a fixture type changed. Useful to adopt changed properties from the fixture type. */
		void OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType);
#endif // WITH_EDITOR

		/** The DMX controlled properties that form this patch */
		TArray<TSharedRef<FRemoteControlDMXControlledProperty>> DMXControlledProperties;
	};
}
