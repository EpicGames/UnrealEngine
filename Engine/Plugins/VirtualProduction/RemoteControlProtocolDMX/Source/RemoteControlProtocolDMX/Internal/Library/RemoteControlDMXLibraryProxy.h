// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "RemoteControlDMXLibraryProxy.generated.h"

struct FRemoteControlProperty;
struct FRemoteControlProtocolEntity;
class UDMXLibrary;
class URemoteControlDMXUserData;
class URemoteControlPreset;
namespace UE::RemoteControl::DMX 
{ 
	class FRemoteControlDMXControlledProperty; 
	class FRemoteControlDMXControlledPropertyPatch;
	class FRemoteControlDMXProtocolEntityObserver;
}

DECLARE_MULTICAST_DELEGATE_OneParam(FRemoteControlDMXPrePropertyPatchesChanged, URemoteControlPreset* /** ChangedPreset */)

/** Class responsible to maintain the DMX Library of a Remote Control Preset */
UCLASS(Transient)
class REMOTECONTROLPROTOCOLDMX_API URemoteControlDMXLibraryProxy : public UObject
{
	GENERATED_BODY()

	using FRemoteControlDMXControlledProperty = UE::RemoteControl::DMX::FRemoteControlDMXControlledProperty;
	using FRemoteControlDMXControlledPropertyPatch = UE::RemoteControl::DMX::FRemoteControlDMXControlledPropertyPatch;
	using FRemoteControlDMXProtocolEntityObserver = UE::RemoteControl::DMX::FRemoteControlDMXProtocolEntityObserver;

public:
	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	//~ End UObject interface

	/** Gets the DMX library of this proxy */
	UDMXLibrary* GetDMXLibrary() const;

	/** Returns the DMX controlled property patches in this proxy */
	TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> GetPropertyPatches() const { return PropertyPatches; }

	/** Refreshes the proxy on end frame */
	void RequestRefresh();

	/** Refreshes the proxy */
	void Refresh();

	/** Resets the proxy so it no longer can receive DMX */
	void Reset();

#if WITH_EDITOR
	/**
	 * Clears Fixture Patches. This should be called after a DMX Library asset in use was force deleted.
	 * Remote Control Protocol Entities keep a 'hard' reference to the DMX Library object, so need to be cleared here
	 */
	void ClearFixturePatches();

	/** Finds fixture patches maintained by this proxy that exceed universe size */
	TArray<UDMXEntityFixturePatch*> FindPatchesThatExceedUniverseSize() const;

	/** Returns a delegate broadcast before property patches are being changed */
	static FRemoteControlDMXPrePropertyPatchesChanged& GetOnPrePropertyPatchesChanged() { return OnPrePropertyPatchesChanged; }

	/** Returns a delegate broadcast after property patches were changed */
	static FSimpleMulticastDelegate& GetOnPostPropertyPatchesChanged() { return OnPostPropertyPatchesChanged; }
#endif // WITH_EDITOR

private:
	/** Updates the property patches used in this proxy */
	void UpdatePropertyPatches();

	/** Binds to the OnFixturePatchReceived event for all patches in use */
	void BindOnFixturePatchesReceived();

	/** Unbinds from the OnFixturePatchReceived event for all patches in use */
	void UnbindOnFixturePatchesReceived();

#if WITH_EDITOR
	/** Updates the Entities Observer */
	void UpdateEntitiesObserver();
#endif

	/** Called when a fixture patch was received */
	UFUNCTION()
	void OnFixturePatchReceived(UDMXEntityFixturePatch* FixturePatch, const FDMXNormalizedAttributeValueMap& ValuePerAttribute);

	/** Returns the current fixture patches in use */
	TArray<UDMXEntityFixturePatch*> GetFixturePatches() const;

	/** Returns exposed properties of the preset that owns this proxy */
	TArray<TSharedRef<FRemoteControlProperty>> GetExposedProperties() const;

	/** Returns the user data that holds this object */
	URemoteControlDMXUserData& GetMutableDMXUserDataChecked();
	
	/** Returns the user data that holds this object */
	const URemoteControlDMXUserData& GetDMXUserDataChecked() const;

	/** Called when the remote control preset was fully loaded */
	void OnPostLoadRemoteControlPreset(URemoteControlPreset* Preset);

	/** Called when an entity was exposed or unexposed */
	void OnEntityExposedOrUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId);

	/** Called when an entity was rebound */
	void OnEntityRebind(const FGuid& EntityId);

	/** Called when entities changed */
	void OnEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities);

	/** Called when a map was fully loaded */
	void OnPostLoadMapWithWorld(UWorld* World);

	/** Current DMX controlled property patches */
	TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> PropertyPatches;

	/** Handle set when the proxy is about to refresh */
	FDelegateHandle RefreshDelegateHandle;

#if WITH_EDITOR
	/** Observes entites for DMX specific property changes in editor */
	TSharedPtr<FRemoteControlDMXProtocolEntityObserver> EntitiesObserver;

	/** Delegate broadcast before property patches are being changed */
	static FRemoteControlDMXPrePropertyPatchesChanged OnPrePropertyPatchesChanged;

	/** Delegate broadcast after property patches were changed */
	static FSimpleMulticastDelegate OnPostPropertyPatchesChanged;
#endif // WITH_EDITOR
};
