// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXEntityReference.h"
#include "UObject/Object.h"

#include "RemoteControlDMXUserData.generated.h"

struct FRemoteControlProtocolEntity;
class UDMXLibrary;
class URemoteControlDMXLibraryProxy;
class URemoteControlPreset;

/** Defines how DMX Protocol Entities should be patched */
UENUM()
enum class ERemoteControlDMXPatchGroupMode : uint8
{
	/** Creates a patch per property */
	GroupByProperty,

	/** Creates a patch per property owner object */
	GroupByOwner,
};

DECLARE_EVENT_OneParam(URemoteControlDMXUserData, FOnRemoteControlDMXUserDataChangedProperty, FPropertyChangedEvent&);

/** DMX user data for a Remote Control Preset */
UCLASS()
class REMOTECONTROLPROTOCOLDMX_API URemoteControlDMXUserData : public UObject
{
	GENERATED_BODY()

	// Allow per preset editor settings to store itself in this DMX user data
	friend class URemoteControlDMXPerPresetEditorSettings;

public:
	URemoteControlDMXUserData();

	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	//~ End UObject interface 

	/** Gets or creates DMX User Data in specified preset. */
	static URemoteControlDMXUserData* GetOrCreateDMXUserData(URemoteControlPreset* InPreset);

	/** Returns the DMX Library held with this user data */
	UDMXLibrary* GetDMXLibrary() const { return DMXLibrary; }

	/** Sets the DMX Library held with this user data. Only valid DMX Libraries can be set. */
	void SetDMXLibrary(UDMXLibrary* NewDMXLibrary);

	/** Returns the DMX Library Proxy */
	URemoteControlDMXLibraryProxy* GetDMXLibraryProxy() const { return DMXLibraryProxy; }

	/** Returns the remote control preset this user data resides in */
	URemoteControlPreset* GetPreset() const;

	/** Sets the patch group mode */
	void SetPatchGroupMode(ERemoteControlDMXPatchGroupMode NewPatchGroupMode);

	/** Returns the current patch mode */
	ERemoteControlDMXPatchGroupMode GetPatchGroupMode() const { return PatchGroupMode; }

	/** Returns true if patches should be auto assigned to a universe and address, otherwise patches are is editable in editor. */
	bool IsAutoPatch() const { return bAutoPatch; }

	/** Sets if auto patching is enabled */
	void SetAutoPatchEnabled(bool bEnabled);

	/** Sets the universe from where patches should be generated. */
	void SetAutoAssignFromUniverse(const int32 NewAutoAssignFromUniverse);

	/** Returns the universe from where patches should be generated. */
	int32 GetAutoAssignFromUniverse() const;

#if WITH_EDITOR
	// Property name getters
	static const FName GetDMXLibraryMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(URemoteControlDMXUserData, DMXLibrary); }
	static const FName GetAutoPatchMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(URemoteControlDMXUserData, DMXLibrary); }
#endif // WITH_EDITOR

private:
	/** Tests if there is a DMX Library, creates a new one if it is null */
	void EnsureValidDMXLibrary();

#if WITH_EDITOR
	/** Upgrades assets created before 5.5 that did not use a DMX library */
	void TryUpgradeFromLegacy();
#endif // WITH_EDITOR

	/** The DMX library the remote control preset uses */
	UPROPERTY(EditAnywhere, Category = "DMX")
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** Defines how DMX Protocol Entities should be patched  */
	UPROPERTY()
	ERemoteControlDMXPatchGroupMode PatchGroupMode = ERemoteControlDMXPatchGroupMode::GroupByOwner;

	/** Proxy to handle the DMX Library */
	UPROPERTY(Transient)
	TObjectPtr<URemoteControlDMXLibraryProxy> DMXLibraryProxy;

	/** The universe from where patches should be generated. */
	UPROPERTY()
	int32 AutoAssignFromUniverse = 1;

	/** When checked, patches are auto assigned to a universe and address, otherwise the patch is editable in editor. */
	UPROPERTY()
	bool bAutoPatch = true;

#if WITH_EDITORONLY_DATA
	/** Generic UObject to store editor data per preset, see URemoteControlDMXPerPresetEditorSettings */
	UPROPERTY()
	TObjectPtr<UObject> PerPresetEditorSettings;
#endif // WITH_EDITORONLY_DATA
};
