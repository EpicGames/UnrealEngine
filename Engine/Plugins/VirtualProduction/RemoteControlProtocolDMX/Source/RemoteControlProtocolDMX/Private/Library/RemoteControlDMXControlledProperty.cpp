// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/RemoteControlDMXControlledProperty.h"

#include "GameFramework/Actor.h"
#include "RemoteControlField.h"
#include "RemoteControlProtocolDMX.h"

namespace UE::RemoteControl::DMX
{
	FRemoteControlDMXControlledProperty::FRemoteControlDMXControlledProperty(const TSharedRef<const FRemoteControlProperty> InExposedProperty)
		: ExposedProperty(InExposedProperty)
	{
		Algo::TransformIf(ExposedProperty->ProtocolBindings, Entities,
			[](const FRemoteControlProtocolBinding& Binding)
			{
				return
					Binding.GetProtocolName() == FRemoteControlProtocolDMX::ProtocolName &&
					Binding.GetRemoteControlProtocolEntityPtr().IsValid() && 
					Binding.GetRemoteControlProtocolEntityPtr()->IsValid();
			},
			[](const FRemoteControlProtocolBinding& Binding)
			{
				return Binding.GetRemoteControlProtocolEntityPtr().ToSharedRef();
			});

		InitializeNewEntities();
		UnifyEntities();
	}

	const UObject* FRemoteControlDMXControlledProperty::GetOwnerActor() const
	{
		const UObject* BoundObject = ExposedProperty->GetBoundObject();
		if (!BoundObject)
		{
			return nullptr;
		}

		if (const AActor* OwnerActor = BoundObject->GetTypedOuter<AActor>())
		{
			return OwnerActor;
		}
		else if (const AActor* Actor = Cast<AActor>(BoundObject))
		{
			return Actor;
		}
		else
		{
			return BoundObject;
		}
	}

	UDMXEntityFixturePatch* FRemoteControlDMXControlledProperty::GetFixturePatch() const
	{
		const FRemoteControlDMXProtocolEntity* FirstDMXEntity = !Entities.IsEmpty() && Entities[0]->IsValid() ? Entities[0]->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		return FirstDMXEntity ? FirstDMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
	}

	int32 FRemoteControlDMXControlledProperty::GetEntityIndexChecked(const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity)
	{
		const int32 EntityIndex = Entities.IndexOfByKey(Entity);
		check(EntityIndex != INDEX_NONE);

		return EntityIndex;
	}

	void FRemoteControlDMXControlledProperty::InitializeNewEntities()
	{
		for (int32 EntityIndex = 0; EntityIndex < Entities.Num(); EntityIndex++)
		{
			FRemoteControlDMXProtocolEntity* DMXEntity = Entities[EntityIndex]->IsValid() ? Entities[EntityIndex]->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
			if (!DMXEntity)
			{
				continue;
			}

			if (DMXEntity->ExtraSetting.FunctionIndex == INDEX_NONE)
			{
				const FString AttributeName = [&DMXEntity, EntityIndex, this]()
					{
						const FString FieldPathInfo = ExposedProperty->FieldPathInfo.ToString();
						if (Entities.Num() == 1)
						{
							// Use the fielpath info for single entities
							return FString::Printf(TEXT("%s"), *FieldPathInfo);
						}
						else if (Entities.Num() <= 3 && 
							(FieldPathInfo.Contains(TEXT("Location")) || FieldPathInfo.Contains(TEXT("Scale"))))
						{							
							// Use XYZ for entities containing "Location" or "Scale" and up to 3 components
							FString VectorComponent;
							if (DMXEntity->HasMask(ERCMask::MaskA))
							{
								VectorComponent += TEXT("X");
							}

							if (DMXEntity->HasMask(ERCMask::MaskB))
							{
								VectorComponent += TEXT("Y");
							}

							if (DMXEntity->HasMask(ERCMask::MaskC))
							{
								VectorComponent += TEXT("Z");
							}

							return VectorComponent.IsEmpty() ? 
								*FieldPathInfo :
								FString::Printf(TEXT("%s_%s"), *FieldPathInfo, *VectorComponent);
						}
						else if (Entities.Num() <= 3 && 
							FieldPathInfo.Contains(TEXT("Rotation")))
						{
							// Use Yaw, Pitch, Roll for entities containing "Rotation" and up to 3 components
							FString VectorComponent;
							if (DMXEntity->HasMask(ERCMask::MaskA))
							{
								VectorComponent += TEXT("Roll");
							}

							if (DMXEntity->HasMask(ERCMask::MaskB))
							{
								VectorComponent += TEXT("Pitch");
							}

							if (DMXEntity->HasMask(ERCMask::MaskC))
							{
								VectorComponent += TEXT("Yaw");
							}

							return VectorComponent.IsEmpty() ?
								*FieldPathInfo : 
								FString::Printf(TEXT("%s_%s"), *FieldPathInfo, *VectorComponent);
						}
						else if (Entities.Num() <= 4 &&
							FieldPathInfo.Contains(TEXT("Color")))
						{
							// Use RGBA for entities containing "Color" and up to 4 components
							FString ColorComponent;
							if (DMXEntity->HasMask(ERCMask::MaskA))
							{
								ColorComponent += TEXT("R");
							}

							if (DMXEntity->HasMask(ERCMask::MaskB))
							{
								ColorComponent += TEXT("G");
							}

							if (DMXEntity->HasMask(ERCMask::MaskC))
							{
								ColorComponent += TEXT("B");
							}


							if (DMXEntity->HasMask(ERCMask::MaskD))
							{
								ColorComponent += TEXT("A");
							}

							return ColorComponent.IsEmpty() ?
								*FieldPathInfo : 
								FString::Printf(TEXT("%s_%s"), *FieldPathInfo, *ColorComponent);
						}
						else
						{
							// For other multi dimensional entities append the entity index
							return FString::Printf(TEXT("%s%i"), *FieldPathInfo, EntityIndex + 1);
						}
					}();

				DMXEntity->ExtraSetting.AttributeName = *AttributeName;
				DMXEntity->ExtraSetting.FunctionIndex = EntityIndex;
			}
		}
	}

	void FRemoteControlDMXControlledProperty::UnifyEntities()
	{
		const FRemoteControlDMXProtocolEntity* FirstDMXEntity = !Entities.IsEmpty() && Entities[0]->IsValid() ? Entities[0]->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		if (!FirstDMXEntity)
		{
			return;
		}

		UDMXEntityFixturePatch* UnifiedFixturePatch = FirstDMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch();
		const bool bUnifiedIsPrimaryPatch = FirstDMXEntity->ExtraSetting.bIsPrimaryPatch;

		for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Entities)
		{
			FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
			if (!DMXEntity || DMXEntity == FirstDMXEntity)
			{
				continue;
			}

			// Only update if the properties differ, as this will trigger another rebuild of the library proxy that uses this object
			if (DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() != UnifiedFixturePatch ||
				DMXEntity->ExtraSetting.bIsPrimaryPatch != bUnifiedIsPrimaryPatch)
			{
				DMXEntity->ExtraSetting.FixturePatchReference = UnifiedFixturePatch;
				DMXEntity->ExtraSetting.bIsPrimaryPatch = bUnifiedIsPrimaryPatch;
			}
		}
	}

	FString FRemoteControlDMXControlledProperty::GetSubobjectPath() const
	{
		if (const UObject* OwnerObject = GetOwnerActor())
		{
			const FString OwnerName = OwnerObject->GetFName().ToString();
			const FString BindingPath = ExposedProperty->GetLastBindingPath().ToString();

			const int32 OwnerNameIndex = BindingPath.Find(OwnerName + TEXT("."));
			return BindingPath.RightChop(OwnerNameIndex + OwnerName.Len() + 1);
		}

		return TEXT("");
	}
}
