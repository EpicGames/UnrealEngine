// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/RemoteControlDMXControlledPropertyPatch.h"

#include "Algo/Find.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/RemoteControlDMXControlledProperty.h"
#include "Library/RemoteControlDMXControlledPropertyPatch.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlProtocolDMX.h"

#define LOCTEXT_NAMESPACE "RemoteControlDMXControlledPropertyPatch"

namespace UE::RemoteControl::DMX
{
	FRemoteControlDMXControlledPropertyPatch::FRemoteControlDMXControlledPropertyPatch(
		URemoteControlDMXUserData& DMXUserData, 
		const TArray<TSharedRef<FRemoteControlDMXControlledProperty>>& InDMXControlledProperties)
		: DMXControlledProperties(InDMXControlledProperties)
	{
#if WITH_EDITOR
		// Listen to fixture type changes in editor
		UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddRaw(this, &FRemoteControlDMXControlledPropertyPatch::OnFixtureTypeChanged);
#endif // WITH_EDITOR
	}

	FRemoteControlDMXControlledPropertyPatch::~FRemoteControlDMXControlledPropertyPatch()
	{
		UDMXEntityFixtureType::GetOnFixtureTypeChanged().RemoveAll(this);
	}

	UDMXEntityFixturePatch* FRemoteControlDMXControlledPropertyPatch::GetFixturePatch() const
	{
		return DMXControlledProperties.IsEmpty() ? nullptr : DMXControlledProperties[0]->GetFixturePatch();
	}

	const UObject* FRemoteControlDMXControlledPropertyPatch::GetOwnerActor() const
	{
		return DMXControlledProperties.IsEmpty() ? nullptr : DMXControlledProperties[0]->GetOwnerActor();
	}

#if WITH_EDITOR
	void FRemoteControlDMXControlledPropertyPatch::OnFixtureTypeChanged(const UDMXEntityFixtureType* ChangedFixtureType)
	{
		UDMXEntityFixturePatch* FixturePatch = GetFixturePatch();
		if (!FixturePatch || FixturePatch->GetFixtureType() != ChangedFixtureType)
		{
			return;
		}

		// Adopt attribute names, data type and signal format if they changed in the library
		for (const TSharedRef<FRemoteControlDMXControlledProperty>& DMXControlledProperty : DMXControlledProperties)
		{
			for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : DMXControlledProperty->GetEntities())
			{
				FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
				const FDMXFixtureMode* ModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;
				if (!DMXEntity || !ModePtr)
				{
					continue;
				}

				const FDMXFixtureFunction* FunctionPtr = Algo::FindBy(ModePtr->Functions, DMXEntity->ExtraSetting.AttributeName, &FDMXFixtureFunction::Attribute);
				if (FunctionPtr)
				{
					// If the attribute exists, update the function index
					DMXEntity->ExtraSetting.FunctionIndex = ModePtr->Functions.IndexOfByPredicate(
						[&FunctionPtr](const FDMXFixtureFunction& Function)
						{
							return &Function == FunctionPtr;
						});
				}

				if (ensureMsgf(ModePtr->Functions.IsValidIndex(DMXEntity->ExtraSetting.FunctionIndex),
					TEXT("Unexpected cannot find function for remote control protocol DMX entity. Cannot update entity.")))
				{
					DMXEntity->ExtraSetting.AttributeName = ModePtr->Functions[DMXEntity->ExtraSetting.FunctionIndex].Attribute.Name;
					DMXEntity->ExtraSetting.DataType = ModePtr->Functions[DMXEntity->ExtraSetting.FunctionIndex].DataType;
					DMXEntity->ExtraSetting.bUseLSB = ModePtr->Functions[DMXEntity->ExtraSetting.FunctionIndex].bUseLSBMode;
				}
			}
		}
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
