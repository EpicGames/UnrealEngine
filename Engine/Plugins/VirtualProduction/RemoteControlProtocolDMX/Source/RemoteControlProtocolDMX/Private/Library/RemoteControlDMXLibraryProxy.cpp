// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/RemoteControlDMXLibraryProxy.h"

#include "Algo/AnyOf.h"
#include "IRemoteControlProtocolModule.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Library/RemoteControlDMXControlledProperty.h"
#include "Library/RemoteControlDMXControlledPropertyPatch.h"
#include "Library/RemoteControlDMXProtocolEntityObserver.h"
#include "Misc/CoreDelegates.h"
#include "RemoteControlDMXLog.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
FRemoteControlDMXPrePropertyPatchesChanged URemoteControlDMXLibraryProxy::OnPrePropertyPatchesChanged;
FSimpleMulticastDelegate URemoteControlDMXLibraryProxy::OnPostPropertyPatchesChanged;
#endif 

void URemoteControlDMXLibraryProxy::PostInitProperties()
{
	Super::PostInitProperties();

	if (IsTemplate())
	{
		return;
	}

	URemoteControlPreset* Preset = Cast<URemoteControlPreset>(GetMutableDMXUserDataChecked().GetOuter());
	if (!ensureMsgf(Preset, TEXT("Invalid outer preset for DMX library proxy. Cannot initialize proxy.")))
	{
		return;
	}

	URemoteControlPreset::OnPostLoadRemoteControlPreset.AddUObject(this, &URemoteControlDMXLibraryProxy::OnPostLoadRemoteControlPreset);
	Preset->OnEntityExposed().AddUObject(this, &URemoteControlDMXLibraryProxy::OnEntityExposedOrUnexposed);
	Preset->OnEntityUnexposed().AddUObject(this, &URemoteControlDMXLibraryProxy::OnEntityExposedOrUnexposed);
	Preset->OnEntityRebind().AddUObject(this, &URemoteControlDMXLibraryProxy::OnEntityRebind);
	Preset->OnEntitiesUpdated().AddUObject(this, &URemoteControlDMXLibraryProxy::OnEntitiesUpdated);

	FCoreUObjectDelegates::PostLoadMapWithWorld.AddUObject(this, &URemoteControlDMXLibraryProxy::OnPostLoadMapWithWorld);
}

void URemoteControlDMXLibraryProxy::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		UpdatePropertyPatches();
	}
}

UDMXLibrary* URemoteControlDMXLibraryProxy::GetDMXLibrary() const
{
	URemoteControlDMXUserData* DMXUserData = Cast<URemoteControlDMXUserData>(GetOuter());
	UDMXLibrary* DMXLibrary = DMXUserData ? DMXUserData->GetDMXLibrary() : nullptr;
	if (!ensureMsgf(DMXUserData, TEXT("Invalid outer for URemoteControlDMXUserData. The proxy expects URemoteControlDMXUserData resp. URemoteControlPreset as its outers.")) ||
		!ensureMsgf(DMXLibrary, TEXT("Unexpected URemoteControlDMXUserData has no valid DMX Library.")))
	{
		return nullptr;
	}

	return DMXLibrary;
}

void URemoteControlDMXLibraryProxy::RequestRefresh()
{
	// Refresh on the next tick
	if (!RefreshDelegateHandle.IsValid())
	{
		UnbindOnFixturePatchesReceived();

		RefreshDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &URemoteControlDMXLibraryProxy::Refresh);
	}
}

void URemoteControlDMXLibraryProxy::Refresh()
{
	using namespace UE::RemoteControl::DMX;

	RefreshDelegateHandle.Reset();
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	// Don't refresh if this object is no longer fully valid
	if (!IsValidChecked(this))
	{
		return;
	}

	URemoteControlDMXUserData* DMXUserData = Cast<URemoteControlDMXUserData>(GetOuter());
	URemoteControlPreset* Preset = DMXUserData ? Cast<URemoteControlPreset>(DMXUserData->GetOuter()) : nullptr;

	if (Preset)
	{
		UnbindOnFixturePatchesReceived();

#if WITH_EDITOR
		// Only refresh editor when the preset is dirty, indicating it changed
		const bool bIsDirty = Preset && Preset->GetPackage() && Preset->GetPackage()->IsDirty();
		if (bIsDirty)
		{
			OnPrePropertyPatchesChanged.Broadcast(Preset);
		}
#endif 

		UpdatePropertyPatches();
		
#if WITH_EDITOR
		if (bIsDirty)
		{
			OnPostPropertyPatchesChanged.Broadcast();

			// Listen to DMX related property changes of entities in editor
			UpdateEntitiesObserver();
		}
#endif 

		BindOnFixturePatchesReceived();
	}
}

void URemoteControlDMXLibraryProxy::Reset()
{
	// Cancel any refresh requests
	RefreshDelegateHandle.Reset();
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	// Reset property patches
	PropertyPatches.Reset();
}

#if WITH_EDITOR
void URemoteControlDMXLibraryProxy::ClearFixturePatches()
{
	for (const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch : PropertyPatches)
	{
		for (const TSharedRef<FRemoteControlDMXControlledProperty>& DMXControlledProperty : PropertyPatch->GetDMXControlledProperties())
		{
			for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : DMXControlledProperty->GetEntities())
			{
				FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
				if (!DMXEntity)
				{
					continue;
				}
				DMXEntity->ExtraSetting.FixturePatchReference = nullptr;
			}
		}
	}
}
#endif 

#if WITH_EDITOR
TArray<UDMXEntityFixturePatch*> URemoteControlDMXLibraryProxy::FindPatchesThatExceedUniverseSize() const
{
	const URemoteControlDMXUserData& DMXUserData = GetDMXUserDataChecked();
	const ERemoteControlDMXPatchGroupMode PatchGroupMode = DMXUserData.GetPatchGroupMode();

	TArray<UDMXEntityFixturePatch*> PatchesThatExceedUniverseSize;
	for (const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch : PropertyPatches)
	{
		UDMXEntityFixturePatch* FixturePatch = PropertyPatch->GetFixturePatch();
		if (FixturePatch && FixturePatch->GetChannelSpan() > DMX_UNIVERSE_SIZE)
		{
			PatchesThatExceedUniverseSize.Add(FixturePatch);
		}
	}

	return PatchesThatExceedUniverseSize;
}
#endif // WITH_EDITOR

void URemoteControlDMXLibraryProxy::UpdatePropertyPatches()
{
	URemoteControlDMXUserData& DMXUserData = GetMutableDMXUserDataChecked();
	UDMXLibrary* DMXLibrary = DMXUserData.GetDMXLibrary();
	if (!ensureMsgf(DMXLibrary, TEXT("Cannot create remote control DMX patches, library is invalid.")))
	{
		return;
	}

	const TSharedPtr<FRemoteControlProtocolDMX> DMXProtocol = StaticCastSharedPtr<FRemoteControlProtocolDMX>(IRemoteControlProtocolModule::Get().GetProtocolByName(FRemoteControlProtocolDMX::ProtocolName));
	if (!ensureMsgf(DMXProtocol.IsValid(), TEXT("Cannot create remote control DMX patches, DMX protocol is not available")))
	{
		return;
	}

	// Checks that for a given binding, its protocol is DMX, and is bound to the DMX protocol.
	auto IsValidDMXBinding =
		[ProtocolBindings = DMXProtocol->GetProtocolBindings()](const FRemoteControlProtocolBinding& InBinding)->bool
		{
			return InBinding.GetProtocolName() == FRemoteControlProtocolDMX::ProtocolName
				&& ProtocolBindings.ContainsByPredicate(FRemoteControlProtocol::CreateProtocolComparator(InBinding.GetPropertyId()));
		};

	// Create DMX Controlled Properties
	TArray<TSharedRef<FRemoteControlProperty>> ExposedProperties = GetExposedProperties();

	TArray<TSharedRef<FRemoteControlDMXControlledProperty>> DMXControlledProperties;
	Algo::TransformIf(ExposedProperties, DMXControlledProperties,
		[&IsValidDMXBinding](const TSharedRef<FRemoteControlProperty>& ExposedProperty)
		{
			const bool bValidPreset = ExposedProperty->GetOwner() != nullptr;
			const bool bValidBoundObject = ExposedProperty->GetBoundObject() != nullptr;
			const bool bHasValidDMXBindings = Algo::AnyOf(ExposedProperty->ProtocolBindings, IsValidDMXBinding);

			return 
				bValidPreset &&
				bValidBoundObject &&
				bHasValidDMXBindings;
		},
		[](const TSharedRef<FRemoteControlProperty>& ExposedProperty)
		{
			return MakeShared<FRemoteControlDMXControlledProperty>(ExposedProperty);
		});

	// Group entities depending on the patch mode
	const ERemoteControlDMXPatchGroupMode PatchGroupMode = DMXUserData.GetPatchGroupMode();

	if (PatchGroupMode == ERemoteControlDMXPatchGroupMode::GroupByProperty)
	{
		TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> PerPropertyPatches;
		Algo::Transform(DMXControlledProperties, PerPropertyPatches,
			[&DMXUserData](const TSharedRef<FRemoteControlDMXControlledProperty>& DMXControlledProperty)
			{
				return MakeShared<FRemoteControlDMXControlledPropertyPatch>(DMXUserData, TArray<TSharedRef<FRemoteControlDMXControlledProperty>>({ DMXControlledProperty }));
			});

		PropertyPatches = PerPropertyPatches;
	}
	else if (PatchGroupMode == ERemoteControlDMXPatchGroupMode::GroupByOwner)
	{
		TMap<const UObject*, TArray<TSharedRef<FRemoteControlDMXControlledProperty>>> OwnerToPropertiesMap;
		for (const TSharedRef<FRemoteControlDMXControlledProperty>& DMXControlledProperty : DMXControlledProperties)
		{
			const UObject* Owner = DMXControlledProperty->GetOwnerActor();
			if (ensureMsgf(Owner, TEXT("Cannot group property by owner. Property has no valid bound object.")))
			{
				OwnerToPropertiesMap.FindOrAdd(Owner).Add(DMXControlledProperty);
			}
		}

		TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> PatchesGroupedByOwner;
		Algo::Transform(OwnerToPropertiesMap, PatchesGroupedByOwner, [&DMXUserData](const TTuple<const UObject*, TArray<TSharedRef<FRemoteControlDMXControlledProperty>>>& ObjectToPropertiesPair)
			{
				return MakeShared<FRemoteControlDMXControlledPropertyPatch>(DMXUserData, ObjectToPropertiesPair.Value);
			});

		PropertyPatches = PatchesGroupedByOwner;
	}
	else
	{
		ensureMsgf(0, TEXT("Unhandled patch mode, cannot create DMX patch for Remote Control Preset"));
	}
}

void URemoteControlDMXLibraryProxy::BindOnFixturePatchesReceived()
{
	TArray<UDMXEntityFixturePatch*> FixturePatches = GetFixturePatches();
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		if (!FixturePatch ||
			FixturePatch->OnFixturePatchReceivedDMX.Contains(this, GET_FUNCTION_NAME_CHECKED(URemoteControlDMXLibraryProxy, OnFixturePatchReceived)))
		{
			continue;
		}

		FixturePatch->OnFixturePatchReceivedDMX.AddDynamic(this, &URemoteControlDMXLibraryProxy::OnFixturePatchReceived);
	}
}

void URemoteControlDMXLibraryProxy::UnbindOnFixturePatchesReceived()
{
	TArray<UDMXEntityFixturePatch*> FixturePatches = GetFixturePatches();
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		if (!FixturePatch ||
			!FixturePatch->OnFixturePatchReceivedDMX.Contains(this, GET_FUNCTION_NAME_CHECKED(URemoteControlDMXLibraryProxy, OnFixturePatchReceived)))
		{
			continue;
		}

		FixturePatch->OnFixturePatchReceivedDMX.RemoveAll(this);
	}
}

#if WITH_EDITOR
void URemoteControlDMXLibraryProxy::UpdateEntitiesObserver()
{
	TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> Entities;
	for (const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch : PropertyPatches)
	{
		for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : PropertyPatch->GetDMXControlledProperties())
		{
			Entities.Append(Property->GetEntities());
		}
	}

	EntitiesObserver = MakeShared<FRemoteControlDMXProtocolEntityObserver>(Entities);
}
#endif // WITH_EDITOR

void URemoteControlDMXLibraryProxy::OnFixturePatchReceived(UDMXEntityFixturePatch* FixturePatch, const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	const TSharedPtr<IRemoteControlProtocol> DMXProtocol = IRemoteControlProtocolModule::Get().GetProtocolByName(FRemoteControlProtocolDMX::ProtocolName);
	if (!DMXProtocol.IsValid())
	{
		return;
	}

	for (const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch : PropertyPatches)
	{
		if (PropertyPatch->GetFixturePatch() != FixturePatch)
		{
			continue;
		}

		for (const TSharedRef<FRemoteControlDMXControlledProperty>& DMXControlledProperty : PropertyPatch->GetDMXControlledProperties())
		{
			for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : DMXControlledProperty->GetEntities())
			{
				FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
				if (!DMXEntity)
				{
					continue;
				}

				bool bSuccess = false;
				const int32 DMXValue = FixturePatch->GetAttributeValue(DMXEntity->ExtraSetting.AttributeName, bSuccess);
				if (bSuccess)
				{
					DMXProtocol->QueueValue(Entity, DMXValue);
				}
			}
		}
	}
}

TArray<UDMXEntityFixturePatch*> URemoteControlDMXLibraryProxy::GetFixturePatches() const
{
	TArray<UDMXEntityFixturePatch*> FixturePatches;
	Algo::TransformIf(PropertyPatches, FixturePatches,
		[](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
		{
			return PropertyPatch->GetFixturePatch() != nullptr;
		},
		[](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
		{
			return PropertyPatch->GetFixturePatch();
		});

	return FixturePatches;
}

TArray<TSharedRef<FRemoteControlProperty>> URemoteControlDMXLibraryProxy::GetExposedProperties() const
{
	URemoteControlDMXUserData* DMXUserData = Cast<URemoteControlDMXUserData>(GetOuter());
	URemoteControlPreset* Preset = DMXUserData ? Cast<URemoteControlPreset>(DMXUserData->GetOuter()) : nullptr;

	UDMXLibrary* DMXLibrary = GetDMXLibrary();
	if (!ensureMsgf(DMXUserData, TEXT("Invalid outer for URemoteControlDMXUserData. The proxy expects URemoteControlDMXUserData resp. URemoteControlPreset as its outers.")) ||
		!ensureMsgf(Preset, TEXT("Unexpected URemoteControlDMXUserData has no valid outer DMX User Data.")))
	{
		return {};
	}

	const TArray<TWeakPtr<FRemoteControlProperty>> WeakExposedProperties = Preset->GetExposedEntities<FRemoteControlProperty>();

	TArray<TSharedRef<FRemoteControlProperty>> ExposedProperties;
	Algo::TransformIf(WeakExposedProperties, ExposedProperties,
		[](const TWeakPtr<FRemoteControlProperty>& WeakExposedProperty)
		{
			return WeakExposedProperty.IsValid();
		},
		[](const TWeakPtr<FRemoteControlProperty>& WeakExposedProperty)
		{
			return WeakExposedProperty.Pin().ToSharedRef();
		});

	return ExposedProperties;
}

const URemoteControlDMXUserData& URemoteControlDMXLibraryProxy::GetDMXUserDataChecked() const
{
	return *CastChecked<URemoteControlDMXUserData>(GetOuter());
}

URemoteControlDMXUserData& URemoteControlDMXLibraryProxy::GetMutableDMXUserDataChecked()
{
	return *CastChecked<URemoteControlDMXUserData>(GetOuter());
}

void URemoteControlDMXLibraryProxy::OnPostLoadRemoteControlPreset(URemoteControlPreset* Preset)
{
	RequestRefresh();
}

void URemoteControlDMXLibraryProxy::OnEntityExposedOrUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId)
{
	RequestRefresh();
}

void URemoteControlDMXLibraryProxy::OnEntityRebind(const FGuid& EntityId)
{
	RequestRefresh();
}

void URemoteControlDMXLibraryProxy::OnEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities)
{
	RequestRefresh();
}

void URemoteControlDMXLibraryProxy::OnPostLoadMapWithWorld(UWorld* World)
{
	RequestRefresh();
}
