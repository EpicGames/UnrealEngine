// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "RemoteControlDMXProtocolEntityObserver.h"

#include "Library/RemoteControlDMXLibraryProxy.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"

namespace UE::RemoteControl::DMX
{
	FRemoteControlDMXProtocolEntityComparator::FRemoteControlDMXProtocolEntityComparator(TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>> InEntity)
		: Entity(InEntity)
	{
		UpdateCache();
	}

	bool FRemoteControlDMXProtocolEntityComparator::DidPropertiesChange() const
	{
		const FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		if (DMXEntity)
		{
			const bool bChanged = 
				DMXEntity->ExtraSetting.bUseLSB != bUseLSBMode ||
				DMXEntity->ExtraSetting.DataType != DataType;

			if (bChanged)
			{
				UpdateCache();
				return true;
			}
		}
	
		return false;
	}

	void FRemoteControlDMXProtocolEntityComparator::UpdateCache() const
	{
		const FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		if (DMXEntity)
		{
			bUseLSBMode = DMXEntity->ExtraSetting.bUseLSB;
			DataType = DMXEntity->ExtraSetting.DataType;
		}
	}

	FRemoteControlDMXProtocolEntityObserver::FRemoteControlDMXProtocolEntityObserver(const TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>>& Entities)
	{
		Algo::Transform(Entities, Comparators,
			[](const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity)
			{
				return FRemoteControlDMXProtocolEntityComparator(Entity);
			});
	}

	void FRemoteControlDMXProtocolEntityObserver::Tick(float DeltaTime)
	{
		for (const FRemoteControlDMXProtocolEntityComparator& Comparator : Comparators)
		{
			if (Comparator.DidPropertiesChange())
			{
				if (URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy(Comparator.Entity))
				{
					DMXLibraryProxy->RequestRefresh();
					return;
				}
			}
		}
	}

	TStatId FRemoteControlDMXProtocolEntityObserver::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRemoteControlDMXProtocolEntityObserver, STATGROUP_Tickables);
	}

	URemoteControlDMXLibraryProxy* FRemoteControlDMXProtocolEntityObserver::GetDMXLibraryProxy(const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity) const
	{
		const FRemoteControlProtocolEntity* EntityPtr = Entity->IsValid() ? Entity->Get() : nullptr;
		URemoteControlPreset* Preset = EntityPtr ? EntityPtr->GetOwner().Get() : nullptr;;
		if (Preset)
		{
			URemoteControlDMXUserData* DMXUserData = URemoteControlDMXUserData::GetOrCreateDMXUserData(Preset);
			return DMXUserData->GetDMXLibraryProxy();
		}

		return nullptr;
	}
}

#endif
