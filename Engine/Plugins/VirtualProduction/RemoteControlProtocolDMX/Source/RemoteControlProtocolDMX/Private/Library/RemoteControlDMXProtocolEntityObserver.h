// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "DMXProtocolTypes.h"
#include "Templates/SharedPointer.h"
#include "Tickable.h"
#include "UObject/StructOnScope.h"

class URemoteControlDMXLibraryProxy;
struct FRemoteControlProtocolEntity;

namespace UE::RemoteControl::DMX
{
	struct FRemoteControlDMXProtocolEntityComparator
	{
		FRemoteControlDMXProtocolEntityComparator(TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>> InEntity);

		/** Returns true if properties changed */
		bool DidPropertiesChange() const;

		/** The entity to compare */
		const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>> Entity;

	private:
		/** Updates cached values */
		void UpdateCache() const;

		/** Cached LSB Mode */
		mutable bool bUseLSBMode = false;

		/** Cached Signal Format */
		mutable EDMXFixtureSignalFormat DataType = EDMXFixtureSignalFormat::E8Bit;
	};

	/** 
	 * Class that observes and notifies about DMX protocol entity changes. 
	 * 
	 * Required since the related RC event URemoteControlPreset::OnExposedPropertiesModified fires on RC changed properties,
	 * however we only want to handle DMX specific property changes. 
	 */
	class FRemoteControlDMXProtocolEntityObserver
		: public FTickableGameObject
		, public TSharedFromThis<FRemoteControlDMXProtocolEntityObserver>
	{
	public:
		FRemoteControlDMXProtocolEntityObserver() = default;
		FRemoteControlDMXProtocolEntityObserver(const TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>>& Entities);

	private:
		//~ Begin FTickableGameObject interface
		virtual bool IsTickableInEditor() const override { return true; }
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		//~ End FTickableGameObject interface
		
		/** Returns the DMX Library Proxy for the specified Entity */
		URemoteControlDMXLibraryProxy* GetDMXLibraryProxy(const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity) const;

		/** Per entity comparator to detect changes */
		TArray<FRemoteControlDMXProtocolEntityComparator> Comparators;
	};
}

#endif
