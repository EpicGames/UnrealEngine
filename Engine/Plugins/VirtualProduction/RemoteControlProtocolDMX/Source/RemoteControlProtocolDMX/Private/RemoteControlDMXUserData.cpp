// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXUserData.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Library/RemoteControlDMXLibraryProxy.h"
#include "RemoteControlProtocolDMX.h"
#include "RemoteControlPreset.h"

URemoteControlDMXUserData::URemoteControlDMXUserData()
	: PatchGroupMode(ERemoteControlDMXPatchGroupMode::GroupByOwner)
{
	DMXLibrary = CreateDefaultSubobject<UDMXLibrary>("Internal");
	DMXLibraryProxy = CreateDefaultSubobject<URemoteControlDMXLibraryProxy>("DMXLibraryProxy");
}

#if WITH_EDITOR
void URemoteControlDMXUserData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URemoteControlDMXUserData, DMXLibrary))
	{
		// Ensure a valid DMX Lbirary exists in case it was cleared
		EnsureValidDMXLibrary();

		check(DMXLibraryProxy);
		DMXLibraryProxy->RequestRefresh();
	}
}
#endif // WITH_EDITOR

void URemoteControlDMXUserData::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{		
		// Handle cases where the DMX Library is no longer valid.
		// This case can occur when force deleting the DMX Library and restarting the engine without saving the remote control preset. 
		EnsureValidDMXLibrary();

		check(DMXLibraryProxy);
		DMXLibraryProxy->RequestRefresh();
	}
}

void URemoteControlDMXUserData::BeginDestroy()
{
	Super::BeginDestroy();

	if (!IsTemplate())
	{
		check(DMXLibraryProxy);
		DMXLibraryProxy->Reset();
	}
}

URemoteControlDMXUserData* URemoteControlDMXUserData::GetOrCreateDMXUserData(URemoteControlPreset* InPreset)
{
	if (!InPreset)
	{
		return nullptr;
	}

	const TObjectPtr<UObject>* DMXUserDataObjectPtr = Algo::FindByPredicate(InPreset->UserData, [](const TObjectPtr<UObject>& Object)
		{
			return Object && Object->GetClass() == URemoteControlDMXUserData::StaticClass();
		});

	URemoteControlDMXUserData* DMXUserData = [DMXUserDataObjectPtr, &InPreset]()
		{
			if (DMXUserDataObjectPtr)
			{
				return CastChecked<URemoteControlDMXUserData>(*DMXUserDataObjectPtr);
			}
			else
			{
				URemoteControlDMXUserData* NewDMXUserData = NewObject<URemoteControlDMXUserData>(InPreset);
				InPreset->UserData.Add(NewDMXUserData);

#if WITH_EDITOR
				NewDMXUserData->TryUpgradeFromLegacy();
#endif // WITH_EDITOR

				return NewDMXUserData;
			}
		}();

	return DMXUserData;
}

void URemoteControlDMXUserData::SetDMXLibrary(UDMXLibrary* NewDMXLibrary)
{
	if (ensureMsgf(NewDMXLibrary, TEXT("URemoteControlDMXUserData::SetDMXLibrary should not be called with null DMX Libraries. Ignoring call")))
	{
#if WITH_EDITOR
		Modify();
#endif

		DMXLibrary = NewDMXLibrary;

		check(DMXLibraryProxy);
		DMXLibraryProxy->RequestRefresh();
	}
}

URemoteControlPreset* URemoteControlDMXUserData::GetPreset() const
{
	return Cast<URemoteControlPreset>(GetOuter());
}

void URemoteControlDMXUserData::SetAutoPatchEnabled(bool bEnabled)
{
	bAutoPatch = bEnabled;

	if (bAutoPatch)
	{
		check(DMXLibraryProxy);
		DMXLibraryProxy->RequestRefresh();
	}
}

int32 URemoteControlDMXUserData::GetAutoAssignFromUniverse() const
{
	// Only use the auto assign from universe property when in auto patch mode
	return bAutoPatch ? AutoAssignFromUniverse : 1;
}

void URemoteControlDMXUserData::SetAutoAssignFromUniverse(const int32 NewAutoAssignFromUniverse)
{
	if (AutoAssignFromUniverse != NewAutoAssignFromUniverse)
	{
		AutoAssignFromUniverse = FMath::Max(1, NewAutoAssignFromUniverse);

		check(DMXLibraryProxy);
		DMXLibraryProxy->RequestRefresh();
	}
}

void URemoteControlDMXUserData::SetPatchGroupMode(ERemoteControlDMXPatchGroupMode NewPatchGroupMode)
{
	PatchGroupMode = NewPatchGroupMode;
	
	check(DMXLibraryProxy);
	DMXLibraryProxy->RequestRefresh();
}

void URemoteControlDMXUserData::EnsureValidDMXLibrary()
{
	if (!DMXLibrary)
	{
		const FName UnqiueName = MakeUniqueObjectName(this, UDMXLibrary::StaticClass(), "Internal");
		DMXLibrary = NewObject<UDMXLibrary>(this, UnqiueName, RF_Public | RF_Transactional);

#if WITH_EDITOR
		// It is important to clear the fixture patches for the DMX Library proxy when the DMX Library is reset,
		// this is to handle cases where a DMX Library was force deleted. 
		// Remote Control Protocol Entities keep a 'hard' reference to the DMX Library object, so need to be cleared here
		check(DMXLibraryProxy);
		DMXLibraryProxy->ClearFixturePatches();
#endif // WITH_EDITOR
	}
}

#if WITH_EDITOR
void URemoteControlDMXUserData::TryUpgradeFromLegacy()
{
	// No need to test if this is a template 
	if (IsTemplate())
	{
		return;
	}

	URemoteControlPreset* Preset = Cast<URemoteControlPreset>(GetOuter());
	if (!ensureMsgf(Preset, TEXT("Unexpected invalid outer for DMX user data. Cannot try to upgrade Remote Control preset for DMX.")))
	{
		return;
	}

	const TArray<TWeakPtr<FRemoteControlProperty>> WeakExposedProperties = Preset->GetExposedEntities<FRemoteControlProperty>();
	const bool bIsLegacy = Algo::AnyOf(WeakExposedProperties, [](const TWeakPtr<FRemoteControlProperty>& WeakProperty)
		{
			const TSharedPtr< FRemoteControlProperty> Property = WeakProperty.Pin();
			if (!Property.IsValid())
			{
				return false;
			}

			return Algo::AnyOf(Property->ProtocolBindings, [](const FRemoteControlProtocolBinding& Binding)
				{
					const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> Entity = Binding.GetRemoteControlProtocolEntityPtr();

					const FRemoteControlDMXProtocolEntity* DMXEntity = Entity.IsValid() && Entity->IsValid() ?
						Entity->Cast<FRemoteControlDMXProtocolEntity>() :
						nullptr;

					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return 
						DMXEntity && 
						DMXEntity->ExtraSetting.Universe_DEPRECATED > -1 &&
						DMXEntity->ExtraSetting.StartingChannel_DEPRECATED > -1;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				});
		});

	if (bIsLegacy)
	{
		// Use per property manual patching mode for legacy remote control presets
		bAutoPatch = false;
		PatchGroupMode = ERemoteControlDMXPatchGroupMode::GroupByProperty;

		check(DMXLibraryProxy);
		DMXLibraryProxy->Refresh();
		
		for (const TWeakPtr<FRemoteControlProperty>& WeakProperty : WeakExposedProperties)
		{
			const TSharedPtr< FRemoteControlProperty> Property = WeakProperty.Pin();
			if (!Property.IsValid())
			{
				continue;
			}

			for (const FRemoteControlProtocolBinding& Binding : Property->ProtocolBindings)
			{
				const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> Entity = Binding.GetRemoteControlProtocolEntityPtr();

				FRemoteControlDMXProtocolEntity* DMXEntity = Entity.IsValid() && Entity->IsValid() ?
					Entity->Cast<FRemoteControlDMXProtocolEntity>() :
					nullptr;

				UDMXEntityFixturePatch* FixturePatch = DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
				if (!DMXEntity || !FixturePatch)
				{
					continue;
				}

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				const int32 Universe = DMXEntity->ExtraSetting.Universe_DEPRECATED;
				const int32 StartingChannel = DMXEntity->ExtraSetting.StartingChannel_DEPRECATED;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				const int32 EndingChannel = StartingChannel + FixturePatch->GetChannelSpan() - 1;

				// Assign the fixture patch of the DMX entity as per legacy properties
				if (Universe > 0 &&
					Universe <= DMX_MAX_UNIVERSE &&
					StartingChannel > 0 &&
					StartingChannel <= DMX_UNIVERSE_SIZE &&
					EndingChannel > 0 &&
					EndingChannel <= DMX_UNIVERSE_SIZE)
				{
					FixturePatch->SetUniverseID(Universe);
					FixturePatch->SetStartingChannel(StartingChannel);
				}

				// Reset legacy properties
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				DMXEntity->ExtraSetting.Universe_DEPRECATED = -1;
				DMXEntity->ExtraSetting.StartingChannel_DEPRECATED = -1;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}
}
#endif
