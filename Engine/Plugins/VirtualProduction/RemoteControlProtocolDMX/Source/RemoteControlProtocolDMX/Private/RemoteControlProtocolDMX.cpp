// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolDMX.h"

#include "Algo/Find.h"
#include "Library/RemoteControlDMXControlledProperty.h"
#include "Library/RemoteControlDMXControlledPropertyPatch.h"
#include "Library/RemoteControlDMXLibraryProxy.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMXObjectVersion.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolDMX"

uint8 FRemoteControlDMXProtocolEntity::GetNumDMXChannels() const
{
	return static_cast<uint8>(ExtraSetting.DataType) + 1;
}

uint8 FRemoteControlDMXProtocolEntity::GetRangePropertySize() const
{
	switch (ExtraSetting.DataType)
	{
		default:
		case EDMXFixtureSignalFormat::E8Bit:
			return sizeof(uint8);
			
		case EDMXFixtureSignalFormat::E16Bit:
			return sizeof(uint16);

		// @note 24 bit ints are not available natively, so store as 32bit/4 bytes. This will also affect property clamping.
		case EDMXFixtureSignalFormat::E24Bit:
			return sizeof(uint32);
			
		case EDMXFixtureSignalFormat::E32Bit:
			return sizeof(uint32);
	}
}

const FString& FRemoteControlDMXProtocolEntity::GetRangePropertyMaxValue() const
{
	switch (ExtraSetting.DataType)
	{
		default:
		case EDMXFixtureSignalFormat::E8Bit:
			{
				static const FString UInt8Str = FString::FromInt(TNumericLimits<uint8>::Max());
				return UInt8Str;
			}
				
		case EDMXFixtureSignalFormat::E16Bit:
			{
				static const FString UInt16Str = FString::FromInt(TNumericLimits<uint16>::Max());
				return UInt16Str;
			}

		// @note: This is for the UI so it can be anything, independent of serialization requirements.
		case EDMXFixtureSignalFormat::E24Bit:
			{
				static const FString UInt24Str = FString::FromInt((1 << 24) - 1);
				return UInt24Str;
			}
			
		case EDMXFixtureSignalFormat::E32Bit:
			{
				// FString::FromInt doesn't support values beyond 32 bit signed ints, so use FText::AsNumber
				static const FString UInt32Str = FText::AsNumber(TNumericLimits<uint32>::Max(), &FNumberFormattingOptions::DefaultNoGrouping()).ToString();
				return UInt32Str;
			}
	}
}

void FRemoteControlDMXProtocolEntity::Invalidate()
{
	if (URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy(GetOwner().Get()))
	{
		DMXLibraryProxy->RequestRefresh();
	}
}

TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> FRemoteControlDMXProtocolEntity::GetAllDMXProtocolEntitiesInPreset(URemoteControlPreset* Preset)
{
	using namespace UE::RemoteControl::DMX;

	URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy(Preset);
	if (!DMXLibraryProxy)
	{
		return {};
	}

	TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> AllEntities;
	for (const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& Patch : DMXLibraryProxy->GetPropertyPatches())
	{
		for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : Patch->GetDMXControlledProperties())
		{
			AllEntities.Append(Property->GetEntities());
		}
	}

	return AllEntities;
}

TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> FRemoteControlDMXProtocolEntity::FindEntitiesByProperty(const TSharedRef<FRemoteControlProperty>& Property)
{
	using namespace UE::RemoteControl::DMX;

	URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy(Property->GetOwner());
	if (!DMXLibraryProxy)
	{
		return {};
	}
	
	for (const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& Patch : DMXLibraryProxy->GetPropertyPatches())
	{
		const TSharedRef<FRemoteControlDMXControlledProperty>* DMXControlledPropertyPtr = Algo::FindByPredicate(Patch->GetDMXControlledProperties(),
			[&Property](const TSharedRef<FRemoteControlDMXControlledProperty>& DMXControlledProperty)
			{
				return DMXControlledProperty->ExposedProperty == Property;
			});
		
		if (DMXControlledPropertyPtr)
		{
			return (*DMXControlledPropertyPtr)->GetEntities();
		}
	}

	return {};
}

void FRemoteControlDMXProtocolEntity::BindDMX()
{
	if (URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy(GetOwner().Get()))
	{
		DMXLibraryProxy->RequestRefresh();
	}
}

void FRemoteControlDMXProtocolEntity::UnbindDMX()
{
	if (URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy(GetOwner().Get()))
	{
		DMXLibraryProxy->RequestRefresh();
	}
}

#if WITH_EDITOR
void FRemoteControlDMXProtocolEntity::SetAttributeName(const FName& AttributeName)
{
	ExtraSetting.AttributeName = AttributeName;
}
#endif // WITH_EDITOR

bool FRemoteControlDMXProtocolEntity::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRemoteControlProtocolDMXObjectVersion::GUID);
	
	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FRemoteControlDMXProtocolEntity::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FRemoteControlProtocolDMXObjectVersion::GUID) < FRemoteControlProtocolDMXObjectVersion::MoveRemoteControlProtocolDMXEntityPropertiesToExtraSettingStruct)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Move relevant properties that were moved to the the ExtraSetting member in 5.0 to the ExtraSetting member, so they can be customized.
			ExtraSetting.bUseDefaultInputPort_DEPRECATED = bUseDefaultInputPort_DEPRECATED;
			ExtraSetting.bUseLSB = bUseLSB_DEPRECATED;
			ExtraSetting.DataType = DataType_DEPRECATED;
			ExtraSetting.InputPortId_DEPRECATED = InputPortId_DEPRECATED;
			ExtraSetting.Universe_DEPRECATED = Universe_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif
}

URemoteControlDMXLibraryProxy* FRemoteControlDMXProtocolEntity::GetDMXLibraryProxy(URemoteControlPreset* Preset)
{
	URemoteControlDMXUserData* DMXUserData = URemoteControlDMXUserData::GetOrCreateDMXUserData(Preset);
	return DMXUserData ? DMXUserData->GetDMXLibraryProxy() : nullptr;
}

const FName FRemoteControlProtocolDMX::ProtocolName = "DMX";

#if WITH_EDITOR
const FName FRemoteControlProtocolDMX::PatchColumnName = "Patch";
const FName FRemoteControlProtocolDMX::UniverseColumnName = "Universe";
const FName FRemoteControlProtocolDMX::ChannelColumnName = "Channel";
#endif // WITH_EDITOR

void FRemoteControlProtocolDMX::Bind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr)
{
	if (!ensure(InRemoteControlProtocolEntityPtr.IsValid()))
	{
		return;
	}
	
	FRemoteControlDMXProtocolEntity* DMXProtocolEntity = InRemoteControlProtocolEntityPtr->CastChecked<FRemoteControlDMXProtocolEntity>();
	DMXProtocolEntity->BindDMX();

	FRemoteControlProtocolEntityWeakPtr* ExistingProtocolBindings = WeakProtocolsBindings.FindByPredicate([DMXProtocolEntity](const FRemoteControlProtocolEntityWeakPtr& InProtocolEntity)
		{
			if (const FRemoteControlProtocolEntityPtr& ProtocolEntity = InProtocolEntity.Pin())
			{
				const FRemoteControlDMXProtocolEntity* ComparedDMXProtocolEntity = ProtocolEntity->CastChecked<FRemoteControlDMXProtocolEntity>();

				if (ComparedDMXProtocolEntity->GetPropertyId() == DMXProtocolEntity->GetPropertyId())
				{
					return true;
				}
			}

			return false;
		});

	if (!ExistingProtocolBindings)
	{
		WeakProtocolsBindings.Emplace(InRemoteControlProtocolEntityPtr);
	}
}

void FRemoteControlProtocolDMX::Unbind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr)
{
	if (!ensure(InRemoteControlProtocolEntityPtr.IsValid()))
	{
		return;
	}

	FRemoteControlDMXProtocolEntity* DMXProtocolEntity = InRemoteControlProtocolEntityPtr->CastChecked<FRemoteControlDMXProtocolEntity>();
	DMXProtocolEntity->UnbindDMX();

	WeakProtocolsBindings.RemoveAllSwap(CreateProtocolComparator(DMXProtocolEntity->GetPropertyId()));
}

#if WITH_EDITOR
void FRemoteControlProtocolDMX::RegisterColumns()
{
	FRemoteControlProtocol::RegisterColumns();

	REGISTER_COLUMN(FRemoteControlProtocolDMX::PatchColumnName,
		LOCTEXT("RCPresetPatchColumnHeader", "Patch"),
		ProtocolColumnConstants::ColumnSizeNormal);
}
#endif // WITH_EDITOR

void FRemoteControlProtocolDMX::UnbindAll()
{
	for (const TWeakPtr<TStructOnScope<FRemoteControlProtocolEntity>>& WeakEntity : WeakProtocolsBindings)
	{
		if (!WeakEntity.IsValid())
		{
			continue;
		}
		const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>> Entity = WeakEntity.Pin().ToSharedRef();

		FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		if (!DMXEntity)
		{
			continue;
		}
	}

	WeakProtocolsBindings.Empty();
}

#undef LOCTEXT_NAMESPACE
