// Copyright Epic Games, Inc. All Rights Reserved.


#include "IRemoteControlProtocolModule.h"
#include "RemoteControlProtocolDMX.h"
#include "RemoteControlProtocolDMXSettings.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "RemoteControlProtocolDMXModule"

/**
 * DMX remote control module
 */
class FRemoteControlProtocolDMXModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface

	virtual void StartupModule() override
	{
		const IRemoteControlProtocolModule& RemoteControlProtocolModule = IRemoteControlProtocolModule::Get();
		if (!RemoteControlProtocolModule.IsRCProtocolsDisable())
		{
			IRemoteControlProtocolModule::Get().AddProtocol(FRemoteControlProtocolDMX::ProtocolName, MakeShared<FRemoteControlProtocolDMX>());
		}
	}

	virtual void ShutdownModule() override
	{
	}

	//~ End IModuleInterface
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRemoteControlProtocolDMXModule, RemoteControlProtocolDMX);
