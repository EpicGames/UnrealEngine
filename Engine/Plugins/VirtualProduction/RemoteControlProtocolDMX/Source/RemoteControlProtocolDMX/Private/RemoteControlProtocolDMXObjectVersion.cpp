// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolDMXObjectVersion.h"

#include "Serialization/CustomVersion.h"


const FGuid FRemoteControlProtocolDMXObjectVersion::GUID(0x077636E9, 0x44119FD9, 0x26477F9D, 0x0A7E9F6E);

// Register the custom version with core
FCustomVersionRegistration GRegisterRemoteControlProtocolDMXCustomVersion(FRemoteControlProtocolDMXObjectVersion::GUID, FRemoteControlProtocolDMXObjectVersion::LatestVersion, TEXT("RemoteControlProtocolDMXObjectVersion"));
