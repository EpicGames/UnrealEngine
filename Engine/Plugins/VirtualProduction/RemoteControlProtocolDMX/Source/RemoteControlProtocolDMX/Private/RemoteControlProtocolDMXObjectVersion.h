// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


struct FGuid;

// Custom serialization version for changes to Remote Control Protocol DMX Objects
struct FRemoteControlProtocolDMXObjectVersion
{
	enum Type
	{
		// Roughly corresponds to 4.27
		BeforeCustomVersionWasAdded = 0,

		// Move Remote Control Protocol DMX Entity Properties to the Extra Setting struct so they can be customized
		MoveRemoteControlProtocolDMXEntityPropertiesToExtraSettingStruct,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FRemoteControlProtocolDMXObjectVersion() {}
};
