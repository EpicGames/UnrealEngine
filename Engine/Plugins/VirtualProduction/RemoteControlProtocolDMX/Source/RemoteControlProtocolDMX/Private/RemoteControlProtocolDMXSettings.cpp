// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolDMXSettings.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSimpleMulticastDelegate URemoteControlProtocolDMXSettings::OnRemoteControlProtocolDMXSettingsChangedDelegate_DEPRECATED;

FGuid URemoteControlProtocolDMXSettings::GetOrCreateDefaultInputPortId()
{
	// DEPRECATED 5.5

	return FGuid();
}

FSimpleMulticastDelegate& URemoteControlProtocolDMXSettings::GetOnRemoteControlProtocolDMXSettingsChanged()
{
	return OnRemoteControlProtocolDMXSettingsChangedDelegate_DEPRECATED;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
