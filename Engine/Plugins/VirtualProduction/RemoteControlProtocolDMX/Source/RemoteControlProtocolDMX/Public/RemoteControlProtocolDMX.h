// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlProtocol.h"

#include "DMXProtocolCommon.h"
#include "IO/DMXInputPortReference.h"
#include "Library/DMXEntityFixtureType.h"
#include "RemoteControlProtocolBinding.h"

#include "RemoteControlProtocolDMX.generated.h"

class FRemoteControlProtocolDMX;
class URemoteControlDMXLibraryProxy;
struct FRemoteControlProperty;

/**
 * An inner struct holding DMX specific data.
 * Useful to have type customization for the struct.
 */
USTRUCT()
struct FRemoteControlDMXProtocolEntityExtraSetting
{
	GENERATED_BODY();

	/** Reference to the fixture patch this binding uses */
	UPROPERTY()
	FDMXEntityFixturePatchRef FixturePatchReference;

#if WITH_EDITORONLY_DATA
	/** If true clears the patch instead of generating one when the outer DMX entity is invalidated */
	UPROPERTY(Transient)
	bool bRequestClearPatch = false;
#endif // WITH_EDITORONLY_DATA

	/** 
	 * If true, this entity defines the patch and its fixture type. 
	 * If false, this entity only follows the patch, but does not update the fixture type.
	 */
	UPROPERTY()
	bool bIsPrimaryPatch = true;

	/** The index of the DMX function to receive */
	UPROPERTY()
	int32 FunctionIndex = INDEX_NONE;

	/** The attribute name of this binding */
	UPROPERTY()
	FName AttributeName;

	/**
	 * Least Significant Byte mode makes the individual bytes (channels) of the function be
	 * interpreted with the first bytes being the lowest part of the number.
	 */
	UPROPERTY(EditAnywhere, Category = Mapping)
	bool bUseLSB = false;

	/** Defines the used number of channels (bytes) */
	UPROPERTY(EditAnywhere, Category = Mapping)
	EDMXFixtureSignalFormat DataType = EDMXFixtureSignalFormat::E8Bit;

public:
	// Workaround for clang deprecation warnings for deprecated PackageGuid member in implicit constructors	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRemoteControlDMXProtocolEntityExtraSetting() = default;
	FRemoteControlDMXProtocolEntityExtraSetting(const FRemoteControlDMXProtocolEntityExtraSetting&) = default;
	FRemoteControlDMXProtocolEntityExtraSetting(FRemoteControlDMXProtocolEntityExtraSetting&&) = default;
	FRemoteControlDMXProtocolEntityExtraSetting& operator=(const FRemoteControlDMXProtocolEntityExtraSetting&) = default;
	FRemoteControlDMXProtocolEntityExtraSetting& operator=(FRemoteControlDMXProtocolEntityExtraSetting&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	

	//////////////////////////
	// DEPRECATED PROPERTIES

#if WITH_EDITORONLY_DATA
	/** The DMX universe of this entity. -1 in auto patch mode. */
	UPROPERTY()
	int32 Universe_DEPRECATED = -1;

	/** The starting channel of this entity. -1 in auto patch mode. */
	UPROPERTY()
	int32 StartingChannel_DEPRECATED = -1;

	/** If set to true, uses the default input port set in Remote Control Protocol project settings */
	UE_DEPRECATED(5.5, "Remote control now uses a DMX Library internally. Please refer to the Fixture Patch ref instead.")
	UPROPERTY()
	bool bUseDefaultInputPort_DEPRECATED = true;

	/** Reference of an input DMX port id */
	UE_DEPRECATED(5.5, "Remote control now uses a DMX Library internally. Please refer to the Fixture Patch ref instead.")
	UPROPERTY()
	FGuid InputPortId_DEPRECATED;
#endif 

};

/**
 * DMX protocol entity for remote control binding
 */
USTRUCT()
struct FRemoteControlDMXProtocolEntity : public FRemoteControlProtocolEntity
{
	GENERATED_BODY()

public:
	/** Returns the num DMX channels this setting spans */
	uint8 GetNumDMXChannels() const;

	//~ Begin FRemoteControlProtocolEntity interface
	virtual FName GetRangePropertyName() const override { return NAME_UInt32Property; }
	virtual uint8 GetRangePropertySize() const override;
	virtual const FString& GetRangePropertyMaxValue() const override;
	//~ End FRemoteControlProtocolEntity interface

	/** Invalidates this entity. The entity will be updated on the next tick. */
	REMOTECONTROLPROTOCOLDMX_API void Invalidate();

	/** Finds remote control protocol DMX entities used by the specified property */
	REMOTECONTROLPROTOCOLDMX_API static TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> GetAllDMXProtocolEntitiesInPreset(URemoteControlPreset* Preset);

	/** Finds remote control protocol DMX entities used by the specified property */
	REMOTECONTROLPROTOCOLDMX_API static TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> FindEntitiesByProperty(const TSharedRef<FRemoteControlProperty>& Property);

	/** Binds this entity to DMX */
	void BindDMX();

	/** Unbinds this entity from DMX */
	void UnbindDMX();

#if WITH_EDITOR
	/** 
	 * Sets the attribute name of the entity. 
	 * Use with care, the attribute name needs to exist in the fixture patch's active mode for the entity to be functional.
	 * 
	 * @param AttributeName			The DMX attribute name the entity corresponds to.
	 */
	REMOTECONTROLPROTOCOLDMX_API void SetAttributeName(const FName& AttributeName);
#endif // WITH_EDITOR

	/** Called when the struct is serialized */
	bool Serialize(FArchive& Ar);

	/** Called after the struct is serialized */
	void PostSerialize(const FArchive& Ar);

	/** Extra protocol settings */
	UPROPERTY(EditAnywhere, Category = Mapping, meta = (ShowOnlyInnerProperties))
	FRemoteControlDMXProtocolEntityExtraSetting ExtraSetting;

	/** DMX range input property template, used for binding. */
	UPROPERTY(Transient)
	uint32 RangeInputTemplate = 0;

private:
	/** Gets or creates the DMX library proxy */
	static URemoteControlDMXLibraryProxy* GetDMXLibraryProxy(URemoteControlPreset* Preset);

	// DEPRECATED MEMBERS
public:
#if WITH_EDITORONLY_DATA
	// Deprecated 5.0
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This Property is deprecated and will be removed in a future release. It was moved to the ExtraSetting struct member so the property can be customized."))
	int32 Universe_DEPRECATED = 0;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This Property is deprecated and will be removed in a future release. It was moved to the ExtraSetting struct member so the property can be customized."))
	bool bUseLSB_DEPRECATED = false;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This Property is deprecated and will be removed in a future release. It was moved to the ExtraSetting struct member so the property can be customized."))
	EDMXFixtureSignalFormat DataType_DEPRECATED = EDMXFixtureSignalFormat::E8Bit;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This Property is deprecated and will be removed in a future release. It was moved to the ExtraSetting struct member so the property can be customized."))
	bool bUseDefaultInputPort_DEPRECATED = true;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This Property is deprecated and will be removed in a future release. It was moved to the ExtraSetting struct member so the property can be customized."))
	FGuid InputPortId_DEPRECATED;
#endif
};

template<>
struct TStructOpsTypeTraits<FRemoteControlDMXProtocolEntity> : public TStructOpsTypeTraitsBase2<FRemoteControlDMXProtocolEntity>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true
	};
};

/**
 * DMX protocol implementation for Remote Control
 */
class FRemoteControlProtocolDMX : public FRemoteControlProtocol
{
public:
	FRemoteControlProtocolDMX()
		: FRemoteControlProtocol(ProtocolName)
	{}

	TConstArrayView<FRemoteControlProtocolEntityWeakPtr> GetProtocolBindings() const
	{
		return WeakProtocolsBindings;
	}

	//~ Begin IRemoteControlProtocol interface
	virtual void Bind(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InRemoteControlProtocolEntity) override;
	virtual void Unbind(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InRemoteControlProtocolEntity) override;
	virtual void UnbindAll() override;
	virtual UScriptStruct* GetProtocolScriptStruct() const override { return FRemoteControlDMXProtocolEntity::StaticStruct(); }
	//~ End IRemoteControlProtocol interface

	/** DMX protocol name */
	REMOTECONTROLPROTOCOLDMX_API static const FName ProtocolName;

#if WITH_EDITOR
	REMOTECONTROLPROTOCOLDMX_API static const FName PatchColumnName;
	REMOTECONTROLPROTOCOLDMX_API static const FName UniverseColumnName;
	REMOTECONTROLPROTOCOLDMX_API static const FName ChannelColumnName;
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	/** Populates protocol specific columns. */
	virtual void RegisterColumns() override;
#endif // WITH_EDIOR

private:
	/** Binding for the DMX protocol */
	TArray<FRemoteControlProtocolEntityWeakPtr> WeakProtocolsBindings;
};
