// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "RemoteControlProtocolDMXSettings.generated.h"

class UE_DEPRECATED(5.5, "This class currently only contains deprecated properties and is not registered with project settings.") URemoteControlProtocolDMXSettings;
/**
 * DMX Remote Control Settings.
 * This class currently only contains deprecated properties and is not registered with project settings.
 */
UCLASS(Config = Engine, DefaultConfig)
class REMOTECONTROLPROTOCOLDMX_API URemoteControlProtocolDMXSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "Remote control DMX no longer has an input port, instead please refer to the input port of the DMX library in DMXUserData of the Remote Control Preset")
	FGuid GetOrCreateDefaultInputPortId();

	/** Returns a delegate broadcast whenever the Remote Control Protocol DMX Settings changed */
	UE_DEPRECATED(5.5, "Currently URemoteControlProtocolDMXSettings has no properties that should signal a change.")
	static FSimpleMulticastDelegate& GetOnRemoteControlProtocolDMXSettingsChanged();

#if WITH_EDITOR
	UE_DEPRECATED(5.5, "The DefaultInputPortId property is deprecated and should no longer be accessed")
	static FName GetDefaultInputPortIdPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(URemoteControlProtocolDMXSettings, DefaultInputPortId_DEPRECATED); }
#endif // WITH_EDITOR

private:
	/** DMX Default Device */
	UPROPERTY()
	FGuid DefaultInputPortId_DEPRECATED;

	static FSimpleMulticastDelegate OnRemoteControlProtocolDMXSettingsChangedDelegate_DEPRECATED;
};
