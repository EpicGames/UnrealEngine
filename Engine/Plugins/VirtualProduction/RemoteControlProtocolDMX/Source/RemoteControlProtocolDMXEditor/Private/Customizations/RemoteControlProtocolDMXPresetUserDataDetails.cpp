// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolDMXPresetUserDataDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlProtocolDMXSettings.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolDMXPresetUserDataDetails"

namespace UE::RemoteControl::DMX
{
	TSharedRef<IDetailCustomization> FRemoteControlProtocolDMXPresetUserDataDetails::MakeInstance()
	{
		return MakeShared<FRemoteControlProtocolDMXPresetUserDataDetails>();
	}

	void FRemoteControlProtocolDMXPresetUserDataDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		DMXLibraryHandle = DetailBuilder.GetProperty(URemoteControlDMXUserData::GetDMXLibraryMemberNameChecked());
		DMXLibraryHandle->MarkHiddenByCustomization();
		DMXLibraryHandle->MarkResetToDefaultCustomized();

		constexpr const TCHAR* NoCategoryName = TEXT("nocategory");
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(NoCategoryName);

		CategoryBuilder.AddProperty(DMXLibraryHandle);
	}
}

#undef LOCTEXT_NAMESPACE
