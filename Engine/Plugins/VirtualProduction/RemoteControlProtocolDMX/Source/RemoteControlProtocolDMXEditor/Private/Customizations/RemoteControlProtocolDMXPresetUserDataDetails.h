// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IPropertyHandle;

namespace UE::RemoteControl::DMX
{
	/** Details customization for URemoteControlDMXPresetUserData */
	class FRemoteControlProtocolDMXPresetUserDataDetails final : public IDetailCustomization
	{
	public:
		/** Creates a new instance of this details customization. */
		static TSharedRef<IDetailCustomization> MakeInstance();

		//~ Begin IDetailCustomization interface
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		//~ End IDetailCustomization interface

	private:
		/** Property handle for the DMXLibrary property */
		TSharedPtr<IPropertyHandle> DMXLibraryHandle;
	};
}
