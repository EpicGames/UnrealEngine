// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXLibraryBuilder.h"

#include "Algo/AnyOf.h"
#include "Algo/MaxElement.h"
#include "Editor.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Library/RemoteControlDMXControlledPropertyPatch.h"
#include "Library/RemoteControlDMXLibraryProxy.h"
#include "Library/RemoteControlDMXPatchBuilder.h"

namespace UE::RemoteControl::DMX
{
	const FString FRemoteControlDMXLibraryBuilder::RCFixtureGroupTag = TEXT("RCGenerated_PatchGroup: ");

	void FRemoteControlDMXLibraryBuilder::Register()
	{
		static TSharedRef<FRemoteControlDMXLibraryBuilder> Instance = MakeShared<FRemoteControlDMXLibraryBuilder>();

		URemoteControlDMXLibraryProxy::GetOnPrePropertyPatchesChanged().AddSP(Instance, &FRemoteControlDMXLibraryBuilder::PrePropertyPatchesChanged);
		URemoteControlDMXLibraryProxy::GetOnPostPropertyPatchesChanged().AddSP(Instance, &FRemoteControlDMXLibraryBuilder::PostPropertyPatchesChanged);
	}

	void FRemoteControlDMXLibraryBuilder::PrePropertyPatchesChanged(URemoteControlPreset* InOutPreset)
	{
		// Set the preset to work with 
		Preset = InOutPreset;

		URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy();
		UDMXLibrary* DMXLibrary = GetDMXLibrary();
		if (DMXLibraryProxy && DMXLibrary)
		{
			// Remember fixture patches and fixture types before property patches changed
			const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> PreEditChangePropertyPatches = DMXLibraryProxy->GetPropertyPatches();

			PreviousFixturePatches.Reset();
			Algo::TransformIf(PreEditChangePropertyPatches, PreviousFixturePatches,
				[](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
				{
					return
						PropertyPatch->GetOwnerActor() &&
						PropertyPatch->GetFixturePatch();
				},
				[](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
				{
					return PropertyPatch->GetFixturePatch();
				});

			PreviousFixtureTypes.Reset();
			Algo::TransformIf(PreviousFixturePatches, PreviousFixtureTypes,
				[](const UDMXEntityFixturePatch* FixturePatch)
				{
					return FixturePatch->GetFixtureType();
				},
				[](const UDMXEntityFixturePatch* FixturePatch)
				{
					return FixturePatch->GetFixtureType();
				});
		}
	}

	void FRemoteControlDMXLibraryBuilder::PostPropertyPatchesChanged()
	{
		if (URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy())
		{
			const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> PostEditChangePropertyPatches = DMXLibraryProxy->GetPropertyPatches();
			FRemoteControlDMXPatchBuilder::BuildFixturePatches(AsShared(), PostEditChangePropertyPatches);

			const URemoteControlDMXUserData* DMXUserData = GetDMXUserData();
			const bool bAutoAssign = DMXUserData && DMXUserData->IsAutoPatch();
			if (bAutoAssign)
			{
				AutoAssignFixturePatches(PostEditChangePropertyPatches);
			}
			else
			{
				// Auto assign new patches
				TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>> NewPropertyPatches;
				Algo::TransformIf(PostEditChangePropertyPatches, NewPropertyPatches,
					[this](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
					{
						return
							PropertyPatch->GetFixturePatch() &&
							!PreviousFixturePatches.Contains(PropertyPatch->GetFixturePatch());
					},
					[this](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
					{
						return PropertyPatch;
					});

				AutoAssignFixturePatches(NewPropertyPatches);
			}

			RemoveObsoleteFixturesFromDMXLibrary(PostEditChangePropertyPatches);
		}
	}

	URemoteControlDMXUserData* FRemoteControlDMXLibraryBuilder::GetDMXUserData() const
	{
		return URemoteControlDMXUserData::GetOrCreateDMXUserData(Preset);
	}

	URemoteControlDMXLibraryProxy* FRemoteControlDMXLibraryBuilder::GetDMXLibraryProxy() const
	{
		URemoteControlDMXUserData* DMXUserData = GetDMXUserData();
		return DMXUserData ? DMXUserData->GetDMXLibraryProxy() : nullptr;
	}

	UDMXLibrary* FRemoteControlDMXLibraryBuilder::GetDMXLibrary() const
	{
		URemoteControlDMXLibraryProxy* DMXLibraryProxy = GetDMXLibraryProxy();
		return DMXLibraryProxy ? DMXLibraryProxy->GetDMXLibrary() : nullptr;
	}

	void FRemoteControlDMXLibraryBuilder::RemoveObsoleteFixturesFromDMXLibrary(const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>>& PostEditChangePropertyPatches)
	{
		UDMXLibrary* DMXLibrary = GetDMXLibrary();
		if (DMXLibrary)
		{
			for (UDMXEntityFixturePatch* PreviousFixturePatch : PreviousFixturePatches)
			{
				if (!PreviousFixturePatch || !PreviousFixturePatch->GetParentLibrary())
				{
					continue;
				}

				const bool bFixturePatchStillReferenced = Algo::AnyOf(PostEditChangePropertyPatches,
					[PreviousFixturePatch](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
					{
						if (PropertyPatch->GetFixturePatch() == PreviousFixturePatch)
						{
							return true;
						}

						return false;
					});

				if (!bFixturePatchStillReferenced)
				{
					UDMXEntityFixturePatch::RemoveFixturePatchFromLibrary(PreviousFixturePatch);
				}
			}

			for (UDMXEntityFixtureType* PreviousFixtureType : PreviousFixtureTypes)
			{
				if (!PreviousFixtureType || !PreviousFixtureType->GetParentLibrary())
				{
					continue;
				}

				const TArray<UDMXEntityFixturePatch*> FixturePatchesInLibrary = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();

				const bool bFixtureTypeStillReferenced = Algo::AnyOf(FixturePatchesInLibrary, [PreviousFixtureType](const UDMXEntityFixturePatch* FixturePatch)
					{
						return FixturePatch && FixturePatch->GetFixtureType() == PreviousFixtureType;
					});

				if (!bFixtureTypeStillReferenced)
				{
					UDMXEntityFixtureType::RemoveFixtureTypeFromLibrary(PreviousFixtureType);
				}
			}
		}
	}

	void FRemoteControlDMXLibraryBuilder::AutoAssignFixturePatches(const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>>& PostEditChangePropertyPatches)
	{
		URemoteControlDMXUserData* DMXUserData = GetDMXUserData();
		UDMXLibrary* DMXLibrary = GetDMXLibrary();
		if (!DMXUserData || !DMXLibrary)
		{
			return;
		}

		TArray<UDMXEntityFixturePatch*> FixturePatches;
		Algo::TransformIf(PostEditChangePropertyPatches, FixturePatches,
			[](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
			{
				return PropertyPatch->GetFixturePatch() != nullptr;
			},
			[](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
			{
				return PropertyPatch->GetFixturePatch();
			});

		// To retain auto-assign order for patches for properties in different worlds, use a group index.
		const int32 GroupIndex = GetOrCreateGroupIndex(*DMXLibrary, FixturePatches);
		const FName Tag = *(RCFixtureGroupTag + FString::FromInt(GroupIndex));
		for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
		{
			FixturePatch->CustomTags.AddUnique(Tag);
		}

		// Acquire all RC related patches
		const TMap<int32, TArray<UDMXEntityFixturePatch*>> GroupIndexToRCFixturePatchesMap = GetGroupIndexToRCFixturePatchesMap(*DMXLibrary);

		// Reset patches
		for (const TTuple<int32, TArray<UDMXEntityFixturePatch*>>& GroupIndexToRCFixturePatchesPair : GroupIndexToRCFixturePatchesMap)
		{
			for (UDMXEntityFixturePatch* FixturePatch : GroupIndexToRCFixturePatchesPair.Value)
			{
				FixturePatch->PreEditChange(nullptr);

				FixturePatch->SetStartingChannel(1);
				FixturePatch->SetUniverseID(1);
			}
		}

		// Auto assign all RC related patches
		UDMXEntityFixturePatch* PreviousFixturePatch = nullptr;
		for (const TTuple<int32, TArray<UDMXEntityFixturePatch*>>& GroupIndexToRCFixturePatchesPair : GroupIndexToRCFixturePatchesMap)
		{
			for (UDMXEntityFixturePatch* FixturePatch : GroupIndexToRCFixturePatchesPair.Value)
			{
				const int32 AutoAssingFromUniverse = DMXUserData->GetAutoAssignFromUniverse();
				const int32	DesiredAbsoluteStartingChannel = [PreviousFixturePatch, AutoAssingFromUniverse]()
					{
						if (PreviousFixturePatch && PreviousFixturePatch->GetUniverseID() >= AutoAssingFromUniverse)
						{
							return
								(int64)PreviousFixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE +
								PreviousFixturePatch->GetEndingChannel();
						}
						else
						{
							return (int64)AutoAssingFromUniverse * DMX_UNIVERSE_SIZE;
						}
					}();

				const bool bFitsUniverse = DesiredAbsoluteStartingChannel % DMX_UNIVERSE_SIZE + FixturePatch->GetChannelSpan() <= DMX_UNIVERSE_SIZE;
				const int64 AbsoluteStartingChannel = bFitsUniverse ? DesiredAbsoluteStartingChannel : (DesiredAbsoluteStartingChannel / DMX_UNIVERSE_SIZE + 1) * DMX_UNIVERSE_SIZE;

				const int32 Universe = AbsoluteStartingChannel / DMX_UNIVERSE_SIZE;
				const int32 Channel = AbsoluteStartingChannel % DMX_UNIVERSE_SIZE + 1;

				FixturePatch->SetUniverseID(Universe);
				FixturePatch->SetStartingChannel(Channel);

				FixturePatch->PostEditChange();

				PreviousFixturePatch = FixturePatch;
			}
		}
	}

	TMap<int32, TArray<UDMXEntityFixturePatch*>> FRemoteControlDMXLibraryBuilder::GetGroupIndexToRCFixturePatchesMap(const UDMXLibrary& DMXLibrary) const
	{
		TMap<int32, TArray<UDMXEntityFixturePatch*>> Result;

		const TArray<UDMXEntityFixturePatch*> AllFixturePatchesInLibrary = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixturePatch>();
		for (UDMXEntityFixturePatch* FixturePatch : AllFixturePatchesInLibrary)
		{
			if (!FixturePatch)
			{
				continue;
			}

			const FName* GroupTagPtr = Algo::FindByPredicate(FixturePatch->CustomTags, [](const FName& Tag)
				{
					return Tag.ToString().StartsWith(RCFixtureGroupTag);
				});

			const int32 GroupIndex = GroupTagPtr ? ExtractGroupIndex(*GroupTagPtr) : INDEX_NONE;
			if (GroupIndex != INDEX_NONE)
			{
				Result.FindOrAdd(GroupIndex).Add(FixturePatch);
			}
		}

		Result.KeyStableSort([](const int32 GroupIndexA, const int32 GroupIndexB)
			{
				return GroupIndexA <= GroupIndexB;
			});

		return Result;
	}

	int32 FRemoteControlDMXLibraryBuilder::GetOrCreateGroupIndex(const UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches) const
	{
		const FName* PreviouslySetTagPtr = [&FixturePatches]() -> const FName*
			{
				for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
				{
					const FName* GroupTagPtr = FixturePatch ?
						Algo::FindByPredicate(FixturePatch->CustomTags, [](const FName& Tag)
							{
								return Tag.ToString().StartsWith(RCFixtureGroupTag);
							}) :
						nullptr;

					if (GroupTagPtr)
					{
						return GroupTagPtr;
					}
				}
				return nullptr;
			}();


		const int32 GroupIndex = PreviouslySetTagPtr ? ExtractGroupIndex(*PreviouslySetTagPtr) : INDEX_NONE;
		if (GroupIndex != INDEX_NONE)
		{
			return GroupIndex;
		}
		else
		{
			// Return the next free group index
			const TArray<UDMXEntityFixturePatch*> AllFixturePatchesInLibrary = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			int32 NextGroupIndex = 0;

			for (const UDMXEntityFixturePatch* FixturePatch : AllFixturePatchesInLibrary)
			{
				if (!FixturePatch || FixturePatches.Contains(FixturePatch))
				{
					continue;
				}

				const FName* OtherRCTagPtr = Algo::FindByPredicate(FixturePatch->CustomTags, [](const FName& Tag)
					{
						return Tag.ToString().StartsWith(RCFixtureGroupTag);
					});

				const int32 OtherGroupIndex = OtherRCTagPtr ? ExtractGroupIndex(*OtherRCTagPtr) : INDEX_NONE;
				if (OtherGroupIndex != INDEX_NONE)
				{
					NextGroupIndex = FMath::Max(NextGroupIndex, OtherGroupIndex + 1);
				}
			}

			return NextGroupIndex;
		}
	}

	int32 FRemoteControlDMXLibraryBuilder::ExtractGroupIndex(const FName& Tag) const
	{
		FString TagString = Tag.ToString();
		if (TagString.RemoveFromStart(RCFixtureGroupTag))
		{
			int32 Result;
			if (LexTryParseString(Result, *TagString))
			{
				return Result;
			}
		}

		return INDEX_NONE;
	}
}
