// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/RemoteControlDMXLibraryProxy.h"
#include "Templates/SharedPointer.h"

class UDMXEntityFixtureType;
class UDMXLibrary;
class URemoteControlDMXLibraryProxy;
class URemoteControlPreset;

namespace UE::RemoteControl::DMX
{
	/** Builds a DMX Library for a Remote Control Preset */
	class FRemoteControlDMXLibraryBuilder
		: public TSharedFromThis<FRemoteControlDMXLibraryBuilder>
	{
	public:
		/** Creates an instance of the DMX Library builder */
		static void Register();

		/** Returns the DMX User Data of the Remote Control Preset */
		URemoteControlDMXUserData* GetDMXUserData() const;

		/** Returns the DMX Library Proxy of the Remote Control Preset */
		URemoteControlDMXLibraryProxy* GetDMXLibraryProxy() const;

		/** Returns the DMX Library of used with this Remote Control Preset */
		UDMXLibrary* GetDMXLibrary() const;

	private:
		/** Called before property patches are being changed */
		void PrePropertyPatchesChanged(URemoteControlPreset* InOutPreset);

		/** Called after property patches were changed */
		void PostPropertyPatchesChanged();

		/** Removes unused Fixture Types and Fixture Patches from the DMX Library */
		void RemoveObsoleteFixturesFromDMXLibrary(const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>>& PostEditChangePropertyPatches);

		/** Auto assigns fixture patches in the DMX Library */
		void AutoAssignFixturePatches(const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>>& PostEditChangePropertyPatches);

		/** Returns all Remote Control related Fixture Patches in the DMX Library, grouped by group index */
		TMap<int32, TArray<UDMXEntityFixturePatch*>> GetGroupIndexToRCFixturePatchesMap(const UDMXLibrary& DMXLibrary) const;
		
		/** Gets or creates a Group Index for a group of Fixture Patches. */
		int32 GetOrCreateGroupIndex(const UDMXLibrary& DMXLibrary, const TArray<UDMXEntityFixturePatch*>& FixturePatches) const;

		/** Extracts a group index from a Fixture Patch tag */
		int32 ExtractGroupIndex(const FName& Tag) const;

		/** Fixture types used before property patches changed */
		TArray<UDMXEntityFixtureType*> PreviousFixtureTypes;

		/** Fixture patches used before property patches changed */
		TArray<UDMXEntityFixturePatch*> PreviousFixturePatches;

		/** The remote control preset for which the Library is built */
		URemoteControlPreset* Preset = nullptr;

		/** Tag for RC Fixture Groups */
		static const FString RCFixtureGroupTag;
	};
}
