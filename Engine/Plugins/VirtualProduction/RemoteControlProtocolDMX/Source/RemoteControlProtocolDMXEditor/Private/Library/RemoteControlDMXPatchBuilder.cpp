// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXPatchBuilder.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/MaxElement.h"
#include "DMXRuntimeUtils.h"
#include "GameFramework/Actor.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Library/RemoteControlDMXControlledProperty.h"
#include "Library/RemoteControlDMXControlledPropertyPatch.h"
#include "Library/RemoteControlDMXLibraryBuilder.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlField.h"
#include "RemoteControlProtocolDMX.h"

#define LOCTEXT_NAMESPACE "FRemoteControlDMXPatchBuilder"

namespace UE::RemoteControl::DMX
{
	namespace Internal
	{
		/** Struct holding data that needs to be transported from an old to a new patch */
		struct FRCDMXFixturePatchData
		{
			FGuid MVRFixtureUUID;
			int32 FixtureID = 1;
		};

		/** Builds a single fixture patch and related fixture type if required from a property patch. */
		class FRCSinglePatchBuilder
			: FNoncopyable
		{
		public:
			/** Constructs a single fixture patch and related fixture type if required from a property patch. */
			FRCSinglePatchBuilder(
				const TSharedRef<FRemoteControlDMXLibraryBuilder>& InDMXLibraryBuilder,
				const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& InDMXControlledPropertyPatch);

			/** Rebuilds a fixture patch in the DMX Library */
			void RebuildFixturePatch();

			/** Returns true if this builder builds a primary patch */
			bool IsPrimary() const;

			/** Returns true if the patch should be cleared */
			bool ShouldClearPatch() const;

			/** Caches fixture patch data into the FixturePatchData member */
			void CacheFixturePatchData();

			/** Clears all fixture types used by the entities of this patch */
			void ClearFixtureTypes();

			/** Clears all fixture patches  by the entities of this patch */
			void ClearFixturePatches();

			/** Updates a primary fixture patch */
			void UpdatePrimaryFixturePatch();

			/** Updates a secondary fixture patch */
			void UpdateSecondaryFixturePatch();

			/** Finds a fixture type that corresponds with this property patch */
			UDMXEntityFixtureType* FindFixtureType(UDMXLibrary& DMXLibrary);

			/** Updates an existing fixture type that corresponds with this property patch */
			void UpdateFixtureType(UDMXLibrary& InDMXLibrary, UDMXEntityFixtureType*& InOutFixtureType);

			/** Finds a fixture patch that corresponds with this property patch */
			UDMXEntityFixturePatch* FindFixturePatch(UDMXEntityFixtureType* FixtureType) const;

			/** Updates an existing fixture patch that corresponds with this property patch */
			void UpdateFixturePatch(UDMXLibrary& DMXLibrary, UDMXEntityFixtureType* FixtureType, UDMXEntityFixturePatch* ReuseFixturePatch);

			/** Sets the fixture patch for this property patch. */
			void SetFixturePatch(UDMXEntityFixturePatch* PrimaryFixturePatch);

			/** Sets if the patch is a primary fixture patch */
			void SetIsPrimaryFixturePatch(bool bIsPrimaryFixturePatch);

			/** Returns the fixture patches currently stored in DMXControlledProperties. Useful when rebuilding the patch from new properties. */
			TArray<UDMXEntityFixturePatch*> GetFixturePatchesFromProperties() const;

			/** Returns the desired name for the patch */
			FString GetDesiredName() const;

			/** Returns the name of the owner object */
			FString GetOwnerObjectName() const;

			/** Returns the transform of the owner object or Identity if the owner object has no transform */
			FTransform GetOwnerObjectTransform() const;

			/** Cached data of the fixture patch to use if a patch needs to be regenerated */
			FRCDMXFixturePatchData FixturePatchData;

			/** The DMX controlled properties contained in the patch */
			TArray<TSharedRef<FRemoteControlDMXControlledProperty>> DMXControlledProperties;

			/** The library builder for which the patch is created */
			const TSharedRef<FRemoteControlDMXLibraryBuilder> DMXLibraryBuilder;

			/** The DMX controlled property patch for which a fixture patch should be created */
			const TSharedRef<FRemoteControlDMXControlledPropertyPatch> DMXControlledPropertyPatch;
		};


		FRCSinglePatchBuilder::FRCSinglePatchBuilder(
			const TSharedRef<FRemoteControlDMXLibraryBuilder>& InDMXLibraryBuilder,
			const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& InDMXControlledPropertyPatch)
			: DMXLibraryBuilder(InDMXLibraryBuilder)
			, DMXControlledPropertyPatch(InDMXControlledPropertyPatch)
		{
			DMXControlledProperties = DMXControlledPropertyPatch->GetDMXControlledProperties();

			CacheFixturePatchData();
		}

		void FRCSinglePatchBuilder::RebuildFixturePatch()
		{
			if (ShouldClearPatch())
			{
				ClearFixturePatches();
				ClearFixtureTypes();
			}
			else if (IsPrimary())
			{
				UpdatePrimaryFixturePatch();
			}
			else
			{
				UpdateSecondaryFixturePatch();
			}
		}

		bool FRCSinglePatchBuilder::IsPrimary() const
		{
			return Algo::AnyOf(DMXControlledProperties, [](const TSharedRef<FRemoteControlDMXControlledProperty>& Property)
				{
					const TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>>& Entities = Property->GetEntities();
					const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& FirstEntity = !Entities.IsEmpty() ? Entities[0].ToSharedPtr() : nullptr;
					const FRemoteControlDMXProtocolEntity* FirstDMXEntity = FirstEntity.IsValid() && FirstEntity->IsValid() ? Entities[0]->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;

					return FirstDMXEntity ? FirstDMXEntity->ExtraSetting.bIsPrimaryPatch : true;
				});
		}

		bool FRCSinglePatchBuilder::ShouldClearPatch() const
		{
			return Algo::AnyOf(DMXControlledProperties, [](const TSharedRef<FRemoteControlDMXControlledProperty>& Property)
				{
					const TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>>& Entities = Property->GetEntities();
					const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& FirstEntity = !Entities.IsEmpty() ? Entities[0].ToSharedPtr() : nullptr;
					const FRemoteControlDMXProtocolEntity* FirstDMXEntity = FirstEntity.IsValid() && FirstEntity->IsValid() ? Entities[0]->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;

					return FirstDMXEntity ? FirstDMXEntity->ExtraSetting.bRequestClearPatch : false;
				});
		}

		void FRCSinglePatchBuilder::UpdatePrimaryFixturePatch()
		{
			URemoteControlDMXUserData* DMXUserData = DMXLibraryBuilder->GetDMXUserData();
			URemoteControlDMXLibraryProxy* DMXLibraryProxy = DMXLibraryBuilder->GetDMXLibraryProxy();
			UDMXLibrary* DMXLibrary = DMXLibraryBuilder->GetDMXLibrary();
			if (DMXUserData && DMXLibraryProxy && DMXLibrary)
			{
				// Make sure all entities are set to be a primary patch
				SetIsPrimaryFixturePatch(true);

				// Get or create a fixture type
				UDMXEntityFixtureType* FixtureType = FindFixtureType(*DMXLibrary);
				UpdateFixtureType(*DMXLibrary, FixtureType);

				// Get or create a fixture patch
				UDMXEntityFixturePatch* ReuseFixturePatch = FindFixturePatch(FixtureType);
				UpdateFixturePatch(*DMXLibrary, FixtureType, ReuseFixturePatch);
			}
		}

		void FRCSinglePatchBuilder::UpdateSecondaryFixturePatch()
		{
			UDMXLibrary* DMXLibrary = DMXLibraryBuilder->GetDMXLibrary();
			if (!DMXLibrary)
			{
				return;
			}

			// Find the primary fixture patch by MVR Fixture UUID
			const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
			UDMXEntityFixturePatch* const* PrimaryFixturePatchPtr = Algo::FindBy(FixturePatches, FixturePatchData.MVRFixtureUUID, &UDMXEntityFixturePatch::GetMVRFixtureUUID);

			UDMXEntityFixturePatch* PrimaryFixturePatch = PrimaryFixturePatchPtr ? *PrimaryFixturePatchPtr : nullptr;

			const FDMXFixtureMode* ModePtr = PrimaryFixturePatch ? PrimaryFixturePatch->GetActiveMode() : nullptr;

			// Try to follow the primary or make it a primary
			const bool bCanFollowPrimaryPatch = PrimaryFixturePatch && PrimaryFixturePatch->GetFixtureType() == FindFixtureType(*DMXLibrary);
			if (bCanFollowPrimaryPatch)
			{
				SetIsPrimaryFixturePatch(false);
				SetFixturePatch(PrimaryFixturePatch);
			}
			else
			{
				// Change to a primary instead
				SetIsPrimaryFixturePatch(true);
				UpdatePrimaryFixturePatch();
			}
		}

		void FRCSinglePatchBuilder::CacheFixturePatchData()
		{
			if (const UDMXEntityFixturePatch* FixturePatch = DMXControlledPropertyPatch->GetFixturePatch())
			{
				FixturePatchData.MVRFixtureUUID = FixturePatch->GetMVRFixtureUUID();
				FixturePatchData.FixtureID = FixturePatch->GetFixtureID();
			}
			else
			{
				FixturePatchData.MVRFixtureUUID = FGuid::NewGuid();
				FixturePatchData.FixtureID = 1;
			}
		}

		void FRCSinglePatchBuilder::ClearFixtureTypes()
		{
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Property->GetEntities())
				{
					FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
					UDMXEntityFixturePatch* FixturePatch = DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
					UDMXEntityFixtureType* FixtureType = FixturePatch ? FixturePatch->GetFixtureType() : nullptr;
					if (FixturePatch)
					{
						UDMXEntityFixtureType::RemoveFixtureTypeFromLibrary(FixtureType);
					}
				}
			}
		}

		void FRCSinglePatchBuilder::ClearFixturePatches()
		{
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Property->GetEntities())
				{
					FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
					UDMXEntityFixturePatch* FixturePatch = DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
					if (FixturePatch)
					{
						UDMXEntityFixturePatch::RemoveFixturePatchFromLibrary(FixturePatch);

						DMXEntity->ExtraSetting.FixturePatchReference = nullptr;
					}
				}
			}
		}

		UDMXEntityFixtureType* FRCSinglePatchBuilder::FindFixtureType(UDMXLibrary& DMXLibrary)
		{
			TArray<const FRemoteControlDMXProtocolEntity*> DMXEntities;
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Property->GetEntities())
				{
					FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
					if (DMXEntity)
					{
						DMXEntities.Add(DMXEntity);
					}
				}
			}

			const TArray<UDMXEntityFixtureType*> FixtureTypes = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixtureType>();
			UDMXEntityFixtureType* const* FixtureTypePtr = Algo::FindByPredicate(FixtureTypes, [&DMXEntities](const UDMXEntityFixtureType* FixtureType)
				{
					if (!FixtureType || FixtureType->Modes.IsEmpty())
					{
						return false;
					}
					const FDMXFixtureMode& Mode = FixtureType->Modes[0];

					if (Mode.Functions.Num() != DMXEntities.Num())
					{
						return false;
					}

					for (int32 FunctionIndex = 0; FunctionIndex < Mode.Functions.Num(); FunctionIndex++)
					{
						const FDMXFixtureFunction& Function = Mode.Functions[FunctionIndex];
						const FRemoteControlDMXProtocolEntity* DMXEntity = DMXEntities.IsValidIndex(FunctionIndex) ? DMXEntities[FunctionIndex] : nullptr;

						if (!DMXEntity)
						{
							continue;
						}

						if (DMXEntity->ExtraSetting.AttributeName != Function.Attribute.Name ||
							DMXEntity->ExtraSetting.DataType != Function.DataType ||
							DMXEntity->ExtraSetting.bUseLSB != Function.bUseLSBMode)
						{
							return false;
						}
					}

					return true;
				});

			return FixtureTypePtr ? *FixtureTypePtr : nullptr;
		}

		void FRCSinglePatchBuilder::UpdateFixtureType(UDMXLibrary& InDMXLibrary, UDMXEntityFixtureType*& InOutFixtureType)
		{
			// Build modes
			FDMXFixtureMode NewMode;
			NewMode.ModeName = TEXT("RemoteControl");

			TMap<FName, int32> AttributeNameToCountMap;
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Property->GetEntities())
				{
					FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
					if (!DMXEntity)
					{
						continue;
					}

					const int32 NextFreeChannel = [&NewMode]()
						{
							const FDMXFixtureFunction* MaxFunctionPtr = Algo::MaxElementBy(NewMode.Functions, &FDMXFixtureFunction::Channel);

							return MaxFunctionPtr ? MaxFunctionPtr->GetLastChannel() + 1 : 1;
						}();

					const FName CleanAttributeName = *DMXEntity->ExtraSetting.AttributeName.GetPlainNameString();

					const int32 AttributeCount = AttributeNameToCountMap.FindOrAdd(CleanAttributeName, 1)++;
					FName AttributeName = AttributeCount > 1 ?
						FName(CleanAttributeName, AttributeCount) :
						DMXEntity->ExtraSetting.AttributeName;

					// Update the attribute name of the entity
					DMXEntity->SetAttributeName(AttributeName);

					// Create the related fixture function
					FDMXFixtureFunction NewFunction;
					NewFunction.Attribute = AttributeName;
					NewFunction.FunctionName = Property->ExposedProperty->FieldPathInfo.ToString();
					NewFunction.Channel = NextFreeChannel;
					NewFunction.DataType = DMXEntity->ExtraSetting.DataType;
					NewFunction.bUseLSBMode = DMXEntity->ExtraSetting.bUseLSB;

					const int32 FunctionIndex = NewMode.Functions.Add(NewFunction);

					// Remember the function index from the fixture function
					DMXEntity->ExtraSetting.FunctionIndex = FunctionIndex;
				}
			}

			// Get or create the fixture type
			if (InOutFixtureType && !InOutFixtureType->Modes.IsEmpty())
			{
				InOutFixtureType->Modes[0] = NewMode;
			}
			else
			{
				// Create a new fixture type
				FDMXEntityFixtureTypeConstructionParams FixtureTypeConstructionParams;
				FixtureTypeConstructionParams.DMXCategory = FDMXFixtureCategory(TEXT("Remote Control"));
				FixtureTypeConstructionParams.ParentDMXLibrary = &InDMXLibrary;
				FixtureTypeConstructionParams.Modes = { NewMode };

				InOutFixtureType = UDMXEntityFixtureType::CreateFixtureTypeInLibrary(FixtureTypeConstructionParams);
				check(InOutFixtureType);

				const FString DesiredFixtureTypeName = TEXT("FT_") + GetDesiredName();
				InOutFixtureType->Name = DesiredFixtureTypeName;
			}

			constexpr int32 RCModeIndex = 0;
			InOutFixtureType->UpdateChannelSpan(RCModeIndex);
		}

		UDMXEntityFixturePatch* FRCSinglePatchBuilder::FindFixturePatch(UDMXEntityFixtureType* FixtureType) const
		{
			const TArray<UDMXEntityFixturePatch*> PreviousFixturePatches = GetFixturePatchesFromProperties();
			if (PreviousFixturePatches.Num() == 1)
			{
				const bool bSameFixtureType =
					PreviousFixturePatches[0] &&
					PreviousFixturePatches[0]->GetFixtureType() &&
					PreviousFixturePatches[0]->GetFixtureType() == FixtureType;

				return bSameFixtureType ? PreviousFixturePatches[0] : nullptr;
			}

			return nullptr;
		}

		void FRCSinglePatchBuilder::UpdateFixturePatch(UDMXLibrary& DMXLibrary, UDMXEntityFixtureType* FixtureType, UDMXEntityFixturePatch* ReuseFixturePatch)
		{
			if (!FixtureType || 
				!IsValid(FixtureType->GetParentLibrary()) || 
				!ReuseFixturePatch)
			{
				ClearFixturePatches();
			}

			UDMXEntityFixturePatch* FixturePatch = ReuseFixturePatch;
			if (!FixturePatch && FixtureType)
			{
				const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary.GetEntitiesTypeCast<UDMXEntityFixturePatch>();
				const bool bMVRFixtureUUIDAlreadyUsed = Algo::AnyOf(FixturePatches, [this](const UDMXEntityFixturePatch* OtherFixturePatch)
					{
						return
							OtherFixturePatch &&
							OtherFixturePatch->GetMVRFixtureUUID() == FixturePatchData.MVRFixtureUUID;
					});

				const FGuid UniqueMVRFixtureUUID = bMVRFixtureUUIDAlreadyUsed ? FGuid::NewGuid() : FixturePatchData.MVRFixtureUUID;

				// Create a new fixture patch
				FDMXEntityFixturePatchConstructionParams FixturePatchConstructionParams;
				FixturePatchConstructionParams.FixtureTypeRef = FDMXEntityFixtureTypeRef(FixtureType);
				FixturePatchConstructionParams.ActiveMode = 0;
				FixturePatchConstructionParams.MVRFixtureUUID = UniqueMVRFixtureUUID;
				FixturePatchConstructionParams.DefaultTransform = GetOwnerObjectTransform();

				FixturePatch = UDMXEntityFixturePatch::CreateFixturePatchInLibrary(FixturePatchConstructionParams);
				check(FixturePatch);

				FixturePatch->Name = FDMXRuntimeUtils::FindUniqueEntityName(&DMXLibrary, UDMXEntityFixturePatch::StaticClass(), GetDesiredName());
			}

			if (FixturePatch && FixtureType)
			{
				FixturePatch->SetFixtureType(FixtureType);
				FixturePatch->GenerateFixtureID(FixturePatchData.FixtureID);

				SetFixturePatch(FixturePatch);
			}
		}

		void FRCSinglePatchBuilder::SetFixturePatch(UDMXEntityFixturePatch* PrimaryFixturePatch)
		{
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Property->GetEntities())
				{
					FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
					if (DMXEntity)
					{
						DMXEntity->ExtraSetting.FixturePatchReference = PrimaryFixturePatch;
					}
				}
			}
		}

		void FRCSinglePatchBuilder::SetIsPrimaryFixturePatch(bool bIsPrimaryFixturePatch)
		{
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Property->GetEntities())
				{
					FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
					if (DMXEntity)
					{
						DMXEntity->ExtraSetting.bIsPrimaryPatch = bIsPrimaryFixturePatch;
					}
				}
			}
		}

		TArray<UDMXEntityFixturePatch*> FRCSinglePatchBuilder::GetFixturePatchesFromProperties() const
		{
			TArray<UDMXEntityFixturePatch*> FixturePatches;
			for (const TSharedRef<FRemoteControlDMXControlledProperty>& Property : DMXControlledProperties)
			{
				if (UDMXEntityFixturePatch* FixturePatch = Property->GetFixturePatch())
				{
					FixturePatches.AddUnique(FixturePatch);
				}
			}

			return FixturePatches;
		}

		FString FRCSinglePatchBuilder::GetDesiredName() const
		{
			URemoteControlDMXUserData* DMXUserData = DMXLibraryBuilder->GetDMXUserData();
			if (DMXUserData)
			{
				ERemoteControlDMXPatchGroupMode PatchGroupMode = DMXUserData->GetPatchGroupMode();

				if (DMXControlledProperties.IsEmpty())
				{
					return TEXT("EmptyPatch");
				}
				else if (PatchGroupMode == ERemoteControlDMXPatchGroupMode::GroupByOwner)
				{
					return GetOwnerObjectName();
				}
				else if (!DMXControlledProperties.IsEmpty())
				{
					return FString::Printf(TEXT("%s_%s"), *GetOwnerObjectName(), *DMXControlledProperties[0]->ExposedProperty->FieldName.ToString());
				}
			}

			return FText(LOCTEXT("InvalidPatchName", "InvalidPatch")).ToString();
		}

		FString FRCSinglePatchBuilder::GetOwnerObjectName() const
		{
			const UObject* OwnerObject = DMXControlledProperties.IsEmpty() ? nullptr : DMXControlledProperties[0]->GetOwnerActor();
			if (!OwnerObject)
			{
				return FText(LOCTEXT("ObjectNotLoadedInfo", "Object is not loaded")).ToString();
			}

			if (const AActor* Actor = Cast<AActor>(OwnerObject))
			{
				return *Actor->GetActorLabel();
			}
			else
			{
				return OwnerObject->GetName();
			}
		}

		FTransform FRCSinglePatchBuilder::GetOwnerObjectTransform() const
		{
			const UObject* OwnerObject = DMXControlledProperties.IsEmpty() ? nullptr : DMXControlledProperties[0]->ExposedProperty->GetBoundObject();
			if (!OwnerObject)
			{
				return FTransform::Identity;
			}

			if (const USceneComponent* SceneComponent = Cast<USceneComponent>(OwnerObject))
			{
				return SceneComponent->GetComponentTransform();
			}
			else if (const USceneComponent* OuterSceneComponent = OwnerObject->GetTypedOuter<USceneComponent>())
			{
				return OuterSceneComponent->GetComponentTransform();
			}
			else if (const AActor* Actor = Cast<AActor>(OwnerObject))
			{
				return Actor->GetTransform();
			}
			else if (const AActor* OuterActor = OwnerObject->GetTypedOuter<AActor>())
			{
				return OuterActor->GetTransform();
			}
			else
			{
				return FTransform::Identity;
			}
		}

	}


	void FRemoteControlDMXPatchBuilder::BuildFixturePatches(
		const TSharedRef<FRemoteControlDMXLibraryBuilder>& InDMXLibraryBuilder,
		const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>>& InDMXControlledPropertyPatches)
	{
		using FRCSinglePatchBuilder = Internal::FRCSinglePatchBuilder;

		// Create patch builder instances for all property patches
		TArray<TSharedRef<FRCSinglePatchBuilder>> Builders;
		Algo::Transform(InDMXControlledPropertyPatches, Builders,
			[&InDMXLibraryBuilder](const TSharedRef<FRemoteControlDMXControlledPropertyPatch>& PropertyPatch)
			{
				return MakeShareable(new FRCSinglePatchBuilder(InDMXLibraryBuilder, PropertyPatch));
			});

		Algo::StableSortBy(Builders, &FRCSinglePatchBuilder::IsPrimary);

		// Update primary fixture patches first, so secondary fixture patches can follow newly created ones
		for (const TSharedRef<FRCSinglePatchBuilder>& Builder : Builders)
		{
			Builder->RebuildFixturePatch();
		}
	}
}

#undef LOCTEXT_NAMESPACE
