// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::RemoteControl::DMX
{
	class FRemoteControlDMXControlledPropertyPatch;
	class FRemoteControlDMXLibraryBuilder;

	/** Builds fixture patches for a DMX Library Builder */
	class FRemoteControlDMXPatchBuilder
	{
	public:
		/** Builds fixture patches from property patches. Updates both the DMX Library and Remote Control Protocol DMX Entities. */
		static void BuildFixturePatches(
			const TSharedRef<FRemoteControlDMXLibraryBuilder>& InDMXLibraryBuilder,
			const TArray<TSharedRef<FRemoteControlDMXControlledPropertyPatch>>& InDMXControlledPropertyPatches);
	};
}
