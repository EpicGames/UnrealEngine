// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXAutoBindHandler.h"

#include "DMXProtocolTypes.h"
#include "IO/DMXInputPort.h"
#include "IRCProtocolBindingList.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"
#include "IRemoteControlProtocolModule.h"

namespace UE::RemoteControl::DMX
{
	void FRemoteControlDMXAutoBindHandler::Register()
	{
		static const FRemoteControlDMXAutoBindHandler Instance = FRemoteControlDMXAutoBindHandler();
	}

	void FRemoteControlDMXAutoBindHandler::Tick(float DeltaTime)
	{
		if (!UObjectInitialized())
		{
			return;
		}

		const IRemoteControlProtocolWidgetsModule& RCWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
		const TSharedPtr<IRCProtocolBindingList> BindingList = RCWidgetsModule.GetProtocolBindingList();
		if (!BindingList.IsValid())
		{
			return;
		}

		URemoteControlPreset* Preset = BindingList.IsValid() ? BindingList->GetPreset() : nullptr;
		const URemoteControlDMXUserData* DMXUserData = URemoteControlDMXUserData::GetOrCreateDMXUserData(Preset);
		if (!DMXUserData)
		{
			return;
		}

		// Only auto bind when auto patching is disabled
		if (!DMXUserData->IsAutoPatch())
		{
			for (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : BindingList->GetAwaitingProtocolEntities())
			{
				ProcessAutoBinding(Entity);
			}
		}
	}

	TStatId FRemoteControlDMXAutoBindHandler::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRemoteControlDMXAutoBindHandler, STATGROUP_Tickables);
	}

	void FRemoteControlDMXAutoBindHandler::ProcessAutoBinding(const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& ProtocolEntity)
	{
		// Bind only in editor 
		if (!GIsEditor)
		{
			return;
		}

		// Bind only when auto-patching is disabled
		const FRemoteControlDMXProtocolEntity* DMXProtocolEntity = ProtocolEntity.IsValid() ? ProtocolEntity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		URemoteControlPreset* Preset = DMXProtocolEntity ? DMXProtocolEntity->GetOwner().Get() : nullptr;
		const URemoteControlDMXUserData* DMXUserData = URemoteControlDMXUserData::GetOrCreateDMXUserData(Preset);
		if (!DMXProtocolEntity ||
			!Preset ||
			!DMXUserData ||
			DMXUserData->IsAutoPatch())
		{
			return;
		}


		UDMXEntityFixturePatch* FixturePatch = DMXProtocolEntity->ExtraSetting.FixturePatchReference.GetFixturePatch();
		const UDMXLibrary* DMXLibrary = FixturePatch ? FixturePatch->GetParentLibrary() : nullptr;
		if (!FixturePatch || !DMXLibrary)
		{
			return;
		}

		// Gather DMX Signals per port
		int32 NumSignals = 0;
		TMap<TSharedRef<FDMXInputPort>, TArray<TSharedRef<FDMXSignal>>> InputPortToSignalsMap;
		for (const TSharedRef<FDMXInputPort>& InputPort : DMXLibrary->GetInputPorts())
		{
			for (const TTuple<int32, TSharedPtr<FDMXSignal>>& ExternUniverseToSignalPair : InputPort->GameThreadGetAllDMXSignals())
			{
				if (ExternUniverseToSignalPair.Value.IsValid())
				{
					InputPortToSignalsMap.FindOrAdd(InputPort).Add(ExternUniverseToSignalPair.Value.ToSharedRef());
					NumSignals++;
				}
			}
		}

		for (const TTuple<TSharedRef<FDMXInputPort>, TArray<TSharedRef<FDMXSignal>>>& InputPortToSignalPair : InputPortToSignalsMap)
		{
			// Ignore first received data 
			const TArray<TSharedRef<FDMXSignal>>* OldSignalsPtr = LastInputPortToSignalsMap.Find(InputPortToSignalPair.Key);
			if (!OldSignalsPtr)
			{
				continue;
			}
			const TArray<TSharedRef<FDMXSignal>>& OldSignals = *OldSignalsPtr;
			const TArray<TSharedRef<FDMXSignal>>& NewSignals = InputPortToSignalPair.Value;

			// Ignore additional and unchanged signals
			if (OldSignals.Num() != NewSignals.Num() ||
				OldSignals == NewSignals)
			{
				continue;
			}

			// Find any changed universe and channel
			bool bFoundAutoBinding = false;
			for (int32 SignalIndex = 0; SignalIndex < OldSignals.Num(); SignalIndex++)
			{
				const TSharedRef<FDMXSignal> OldSignal = OldSignals[SignalIndex];
				const TSharedRef<FDMXSignal> NewSignal = NewSignals[SignalIndex];

				// Only test if data is of same size
				if (OldSignal->ChannelData.IsEmpty() ||
					OldSignal->ChannelData.Num() != NewSignal->ChannelData.Num())
				{
					continue;
				}

				const bool bDataChanged = FMemory::Memcmp(OldSignal->ChannelData.GetData(), NewSignal->ChannelData.GetData(), OldSignal->ChannelData.Num()) != 0;
				if (!bDataChanged)
				{
					continue;
				}

				// Find the changed channel
				TOptional<int32> ChangedChannel;
				for (int32 ChannelIndex = 0; ChannelIndex < OldSignal->ChannelData.Num(); ChannelIndex++)
				{
					if (OldSignal->ChannelData[ChannelIndex] != NewSignal->ChannelData[ChannelIndex])
					{
						ChangedChannel = ChannelIndex + 1;
						break;
					}
				}

				if (ChangedChannel.IsSet())
				{
					const TSharedPtr<IRemoteControlProtocol> Protocol = IRemoteControlProtocolModule::Get().GetProtocolByName(FRemoteControlProtocolDMX::ProtocolName);
					if (!Protocol.IsValid())
					{
						return;
					}

					const int32 Universe = InputPortToSignalPair.Key->ConvertExternToLocalUniverseID(NewSignal->ExternUniverseID);
					Protocol->Unbind(ProtocolEntity);

					FixturePatch->SetUniverseID(Universe);
					FixturePatch->SetStartingChannel(ChangedChannel.GetValue());

					Protocol->Bind(ProtocolEntity);

					bFoundAutoBinding = true;
					break;
				}
			}

			if (bFoundAutoBinding)
			{
				break;
			}
		}

		// Remember the new signals
		LastInputPortToSignalsMap = InputPortToSignalsMap;
	}
}
