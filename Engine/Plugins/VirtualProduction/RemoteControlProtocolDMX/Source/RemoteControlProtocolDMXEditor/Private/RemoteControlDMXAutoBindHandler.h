// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"
#include "Stats/Stats.h"

class FDMXInputPort;
class FDMXSignal;
class URemoteControlPreset;
struct FRemoteControlProtocolEntity;
template<typename T> class TStructOnScope;

namespace UE::RemoteControl::DMX
{
	/** Handles auto binding DMX values and maintains the related list */
	class FRemoteControlDMXAutoBindHandler
		: public FTickableGameObject
	{
	public:
		/** Registers the auto-bind handler with the engine */
		static void Register();

	private:
		//~ Begin FTickableGameObject Interface
		virtual bool IsTickable() const override { return true; }
		virtual bool IsTickableInEditor() const override { return true; }
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		//~ End FTickableGameObject Interface
		
		/**
		 * Processes auto binding
		 * @param InProtocolEntityPtr	Protocol entity pointer
		 */
		void ProcessAutoBinding(const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& ProtocolEntity);

		/** Last received DMX Signals with their input ports */
		TMap<TSharedRef<FDMXInputPort>, TArray<TSharedRef<FDMXSignal>>> LastInputPortToSignalsMap;

		/** Caches if auto patch mode is enabled */
		bool bIsAutoPatch = true;
	};
}
