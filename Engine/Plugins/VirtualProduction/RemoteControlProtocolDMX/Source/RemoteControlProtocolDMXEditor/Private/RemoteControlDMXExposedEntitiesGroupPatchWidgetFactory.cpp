// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXExposedEntitiesGroupPatchWidgetFactory.h"

#include "IRemoteControlUIModule.h"
#include "RemoteControlProtocolDMX.h"
#include "UI/SRemoteControlDMXExposedEntitiesGroupPatch.h"

namespace UE::RemoteControl::DMX
{
	void FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory::Register()
	{
		IRemoteControlUIModule& RemoteControlUIModule = FModuleManager::LoadModuleChecked<IRemoteControlUIModule>("RemoteControlUI");
		RemoteControlUIModule.RegisterExposedEntitiesGroupWidgetFactory(MakeShared<FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory>());
	}

	FName FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory::GetColumnName() const
	{
		return FRemoteControlProtocolDMX::PatchColumnName;
	}

	FName FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory::GetProtocolName() const
	{
		return FRemoteControlProtocolDMX::ProtocolName;
	}

	TSharedRef<SWidget> FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory::MakeWidget(const FRCPanelExposedEntitiesGroupWidgetFactoryArgs& Args)
	{
		return SNew(SRemoteControlDMXExposedEntitiesGroupPatch, Args.WeakPreset, Args.ChildProperties);
	}
}
