// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UI/IRCPanelExposedEntitiesGroupWidgetFactory.h"

class SWidget;
struct FRCFieldGroupColumnExtenderInfo;

namespace UE::RemoteControl::DMX
{
	/** Widget factory the patch column in an exposed entities group row */
	class FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory
		: public IRCPanelExposedEntitiesGroupWidgetFactory
		, public TSharedFromThis<FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory>
	{
	public:
		virtual ~FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory() {};

		/** Registers this factory with the engine */
		static void Register();

	protected:
		//~ Begin IRCPanelExposedEntitiesGroupWidgetFactory interface
		virtual FName GetColumnName() const override;
		virtual FName GetProtocolName() const override;
		virtual TSharedRef<SWidget> MakeWidget(const FRCPanelExposedEntitiesGroupWidgetFactoryArgs& Args) override;
		//~ End IRCPanelExposedEntitiesGroupWidgetFactory
	};
}
