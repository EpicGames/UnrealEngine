// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXExposedEntitiesListSettings.h"

#include "Algo/Find.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IRemoteControlUIModule.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/RemoteControlDMXLibraryProxy.h"
#include "RemoteControlDMXEditorLog.h"
#include "RemoteControlDMXPerPresetEditorSettings.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "RemoteControlDMXExposedEntitiesListSettings"

namespace UE::RemoteControl::DMX
{
	void FRemoteControlDMXExposedEntitiesListSettings::Register()
	{
		static TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol> Instance = MakeShared<FRemoteControlDMXExposedEntitiesListSettings>();
		IRemoteControlUIModule::Get().RegisterExposedEntitiesListSettingsForProtocol(Instance);
	}

	FName FRemoteControlDMXExposedEntitiesListSettings::GetProtocolName() const
	{
		return FRemoteControlProtocolDMX::ProtocolName;
	}

	FRCPanelExposedEntitiesListSettingsData FRemoteControlDMXExposedEntitiesListSettings::GetListSettings(URemoteControlPreset* Preset) const
	{
		const URemoteControlDMXPerPresetEditorSettings* EditorSettings = URemoteControlDMXPerPresetEditorSettings::GetOrCreatePerPresetEditorSettings(Preset);
		if (EditorSettings)
		{
			return EditorSettings->ExposedEntitiesListSettings;
		}

		return FRCPanelExposedEntitiesListSettingsData();
	}
	
	void FRemoteControlDMXExposedEntitiesListSettings::OnSettingsChanged(URemoteControlPreset* Preset, const FRCPanelExposedEntitiesListSettingsData& ListSettings)
	{
		URemoteControlDMXUserData* DMXUserData = GetUserData(Preset);
		URemoteControlDMXLibraryProxy* DMXLibraryProxy = DMXUserData ? DMXUserData->GetDMXLibraryProxy() : nullptr;
		if (Preset && DMXUserData && DMXLibraryProxy)
		{
			// Store in data
			URemoteControlDMXPerPresetEditorSettings* EditorSettings = URemoteControlDMXPerPresetEditorSettings::GetOrCreatePerPresetEditorSettings(Preset);
			if (EditorSettings)
			{
				EditorSettings->Modify();
				EditorSettings->ExposedEntitiesListSettings = ListSettings;
			}

			// Update patch group mode
			const ERemoteControlDMXPatchGroupMode PreviousPatchGroupMode = DMXUserData->GetPatchGroupMode();

			const bool bGroupByOwner = DMXUserData->IsAutoPatch() && ListSettings.FieldGroupType == ERCFieldGroupType::Owner;
			const ERemoteControlDMXPatchGroupMode NewPatchGroupMode = bGroupByOwner ? ERemoteControlDMXPatchGroupMode::GroupByOwner : ERemoteControlDMXPatchGroupMode::GroupByProperty;

			const bool bPatchGroupModeChanged = PreviousPatchGroupMode != NewPatchGroupMode;
			if (bPatchGroupModeChanged)
			{
				DMXUserData->Modify();
				DMXUserData->SetPatchGroupMode(NewPatchGroupMode);
			}

			// Show a warning if not all patches can be created in the current list order
			const TArray<UDMXEntityFixturePatch*> FixturePatchesThatExceedUniverseSize = DMXLibraryProxy->FindPatchesThatExceedUniverseSize();
			if (!FixturePatchesThatExceedUniverseSize.IsEmpty())
			{
				const FText WarningInfo = LOCTEXT("PatchesExceedUniverse", "Remote control generated DMX patches exceed the universe size. See log for details.");

				FNotificationInfo Info(WarningInfo);
				Info.bUseSuccessFailIcons = true;
				Info.ExpireDuration = 10.f;

				FSlateNotificationManager::Get().AddNotification(Info);

				for (const UDMXEntityFixturePatch* FixturePatch : FixturePatchesThatExceedUniverseSize)
				{
					UE_LOGF(LogRemoteControlDMXEditor, Warning, "Remote control generated patch '%ls' exceeds 512 channels. Only the first 512 channels will be available.", *FixturePatch->Name);
				}
			}
		}
	}

	URemoteControlDMXUserData* FRemoteControlDMXExposedEntitiesListSettings::GetUserData(URemoteControlPreset* Preset) const
	{
		if (!Preset)
		{
			return nullptr;
		}
			
		const TObjectPtr<UObject>* DMXUserDataObjectPtr = Algo::FindByPredicate(Preset->UserData, [](const TObjectPtr<UObject>& Object)
			{
				return Object && Object->GetClass() == URemoteControlDMXUserData::StaticClass();
			});

		return DMXUserDataObjectPtr ? CastChecked<URemoteControlDMXUserData>(*DMXUserDataObjectPtr) : nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
