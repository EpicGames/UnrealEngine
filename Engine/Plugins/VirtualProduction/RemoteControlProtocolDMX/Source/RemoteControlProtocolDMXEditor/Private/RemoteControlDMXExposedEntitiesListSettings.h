// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UI/IRCPanelExposedEntitiesListSettingsForProtocol.h"

class URemoteControlDMXUserData;

namespace UE::RemoteControl::DMX
{
	/** Settings for the DMX exposed entities list */
	class FRemoteControlDMXExposedEntitiesListSettings
		: public IRCPanelExposedEntitiesListSettingsForProtocol
		, public TSharedFromThis<FRemoteControlDMXExposedEntitiesListSettings>
	{
	public:
		virtual ~FRemoteControlDMXExposedEntitiesListSettings() = default;

		/** Registers the exposed entities list settings with the engine */
		static void Register();

		//~ Begin IRCPanelExposedEntityListSettings
		virtual FName GetProtocolName() const override;
		virtual FRCPanelExposedEntitiesListSettingsData GetListSettings(URemoteControlPreset* Preset) const override;
		virtual void OnSettingsChanged(URemoteControlPreset* Preset, const FRCPanelExposedEntitiesListSettingsData& ListSettings) override;
		//~ End IRCPanelExposedEntityListSettings

	private:
		/** Returns the DMX user data of the preset */
		URemoteControlDMXUserData* GetUserData(URemoteControlPreset* Preset) const;
	};
}
