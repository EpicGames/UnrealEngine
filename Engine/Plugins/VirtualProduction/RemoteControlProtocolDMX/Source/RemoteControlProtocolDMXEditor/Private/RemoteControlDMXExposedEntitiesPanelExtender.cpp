// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXExposedEntitiesPanelExtender.h"

#include "Algo/Find.h"
#include "IRemoteControlUIModule.h"
#include "Modules/ModuleManager.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"
#include "Styling/AppStyle.h"
#include "UI/SRemoteControlDMXPresetUserData.h"

namespace UE::RemoteControl::DMX
{
	void FRemoteControlDMXExposedEntitiesPanelExtender::Register()
	{
		IRemoteControlUIModule& RemoteControlUIModule = FModuleManager::LoadModuleChecked<IRemoteControlUIModule>("RemoteControlUI");
		RemoteControlUIModule.RegisterExposedEntitiesPanelExtender(MakeShared<FRemoteControlDMXExposedEntitiesPanelExtender>());
	}

	TSharedRef<SWidget> FRemoteControlDMXExposedEntitiesPanelExtender::MakeWidget(URemoteControlPreset* Preset, const IRCExposedEntitiesPanelExtender::FArgs& Args)
	{
		if (!ensureMsgf(Preset, TEXT("Unexpected invalid preset provided when trying to extend the Exposed Entities Panel, cannot create exposed entities panel for DMX")) ||
			!ensureMsgf(Args.ActiveProtocolAttribute.IsBound(), TEXT("Unexpected ActiveProtocolAttribute is not bound, cannot create exposed entities panel for DMX.")))
		{
			return SNullWidget::NullWidget;
		}

		// Get or create DMX user data for this preset
		URemoteControlDMXUserData* DMXUserData = [Preset]()
			{
				TObjectPtr<UObject>* DMXUserDataObjectPtr = Algo::FindByPredicate(Preset->UserData, [](const TObjectPtr<UObject>& Object)
					{
						return Object && Object->GetClass() == URemoteControlDMXUserData::StaticClass();
					});

				if (DMXUserDataObjectPtr)
				{
					return CastChecked<URemoteControlDMXUserData>(*DMXUserDataObjectPtr);
				}
				else
				{
					URemoteControlDMXUserData* NewDMXUserData = NewObject<URemoteControlDMXUserData>(Preset);

					Preset->Modify();
					Preset->UserData.Add(NewDMXUserData);

					return NewDMXUserData;
				}
			}();
			check(DMXUserData);

		// Create the extension widget
		const TSharedRef<SWidget> ExtensionWidget =
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Visibility_Lambda([Args]()
				{
					const bool bIsActiveProtocolDMX = Args.ActiveProtocolAttribute.IsBound() && Args.ActiveProtocolAttribute.Get() == FRemoteControlProtocolDMX::ProtocolName;
					return bIsActiveProtocolDMX ? EVisibility::Visible : EVisibility::Collapsed;
				})
			[
				SNew(SRemoteControlDMXPresetUserData, DMXUserData)
			];

		return ExtensionWidget;
	}
}
