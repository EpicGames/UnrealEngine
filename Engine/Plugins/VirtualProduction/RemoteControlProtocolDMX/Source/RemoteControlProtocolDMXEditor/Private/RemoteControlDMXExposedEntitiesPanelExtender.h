// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UI/IRCExposedEntitiesPanelExtender.h"

class URemoteControlPreset;

namespace UE::RemoteControl::DMX
{
	/** Handles extending the exposed entities panel for DMX */
	class FRemoteControlDMXExposedEntitiesPanelExtender
		: public IRCExposedEntitiesPanelExtender
		, public TSharedFromThis<FRemoteControlDMXExposedEntitiesPanelExtender>
	{
	public:
		virtual ~FRemoteControlDMXExposedEntitiesPanelExtender() {};

		/** Registers the extender with the engine */
		static void Register();

	protected:
		//~ Begin IRCExposedEntitiesPanelExtender interface
		virtual TSharedRef<SWidget> MakeWidget(URemoteControlPreset* Preset, const IRCExposedEntitiesPanelExtender::FArgs& Args) override;
		//~ End IRCExposedEntitiesPanelExtender
	};
}
