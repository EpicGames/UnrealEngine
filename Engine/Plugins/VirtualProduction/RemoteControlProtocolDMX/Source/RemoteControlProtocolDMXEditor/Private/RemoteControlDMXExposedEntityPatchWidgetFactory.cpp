// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXExposedEntityPatchWidgetFactory.h"

#include "IRemoteControlUIModule.h"
#include "RemoteControlProtocolDMX.h"
#include "UI/SRemoteControlDMXExposedEntityPatch.h"

namespace UE::RemoteControl::DMX
{
	void FRemoteControlDMXExposedEntityPatchWidgetFactory::Register()
	{
		IRemoteControlUIModule& RemoteControlUIModule = FModuleManager::LoadModuleChecked<IRemoteControlUIModule>("RemoteControlUI");
		RemoteControlUIModule.RegisterExposedEntityWidgetFactory(MakeShared<FRemoteControlDMXExposedEntityPatchWidgetFactory>());
	}

	FName FRemoteControlDMXExposedEntityPatchWidgetFactory::GetColumnName() const
	{
		return FRemoteControlProtocolDMX::PatchColumnName;
	}

	FName FRemoteControlDMXExposedEntityPatchWidgetFactory::GetProtocolName() const
	{
		return FRemoteControlProtocolDMX::ProtocolName;
	}

	TSharedRef<SWidget> FRemoteControlDMXExposedEntityPatchWidgetFactory::MakePropertyWidget(const FRCPanelExposedPropertyWidgetArgs& Args)
	{
		return SNew(SRemoteControlDMXExposedEntityPatch, Args.WeakPreset, Args.Property);
	}
}
