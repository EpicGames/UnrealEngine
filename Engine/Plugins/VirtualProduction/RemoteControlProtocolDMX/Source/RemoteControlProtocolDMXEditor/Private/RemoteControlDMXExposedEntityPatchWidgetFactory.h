// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/IRCPanelExposedEntityWidgetFactory.h"

namespace UE::RemoteControl::DMX
{
	/** Widget factory the patch column in an exposed entities group row */
	class FRemoteControlDMXExposedEntityPatchWidgetFactory
		: public IRCPanelExposedEntityWidgetFactory
		, public TSharedFromThis<FRemoteControlDMXExposedEntityPatchWidgetFactory>
	{
	public:
		virtual ~FRemoteControlDMXExposedEntityPatchWidgetFactory() {};

		/** Registers the factory with the engine */
		static void Register();

	protected:
		//~ Begin IRCPanelExposedEntitiesGroupWidgetFactory interface
		virtual FName GetColumnName() const override;
		virtual FName GetProtocolName() const override;
		virtual TSharedRef<SWidget> MakePropertyWidget(const FRCPanelExposedPropertyWidgetArgs& Args) override;
		//~ End IRCPanelExposedEntitiesGroupWidgetFactory
	};
}
