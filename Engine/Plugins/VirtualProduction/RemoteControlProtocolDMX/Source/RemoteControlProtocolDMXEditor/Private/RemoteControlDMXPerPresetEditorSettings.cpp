// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDMXPerPresetEditorSettings.h"

#include "RemoteControlDMXUserData.h"
#include "UI/RCPanelExposedEntitiesListSettingsData.h"

URemoteControlDMXPerPresetEditorSettings* URemoteControlDMXPerPresetEditorSettings::GetOrCreatePerPresetEditorSettings(URemoteControlPreset* Preset)
{
	URemoteControlDMXUserData* DMXUserData = URemoteControlDMXUserData::GetOrCreateDMXUserData(Preset);

	if (DMXUserData->PerPresetEditorSettings && DMXUserData->PerPresetEditorSettings->GetClass() == URemoteControlDMXPerPresetEditorSettings::StaticClass())
	{
		return CastChecked<URemoteControlDMXPerPresetEditorSettings>(DMXUserData->PerPresetEditorSettings);
	}
	else
	{
		URemoteControlDMXPerPresetEditorSettings* NewSettings = NewObject<URemoteControlDMXPerPresetEditorSettings>(DMXUserData, NAME_None);
		
		// Set default settings for Remote Control DMX.
		NewSettings->ExposedEntitiesListSettings.FieldGroupType = ERCFieldGroupType::Owner;
		NewSettings->ExposedEntitiesListSettings.FieldGroupOrder = ERCFieldGroupOrder::Ascending;

		// Store it in DMX User Data
		DMXUserData->PerPresetEditorSettings = NewSettings;

		return CastChecked<URemoteControlDMXPerPresetEditorSettings>(DMXUserData->PerPresetEditorSettings);
	}
}
