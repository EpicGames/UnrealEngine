// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/RCPanelExposedEntitiesListSettingsData.h"
#include "UObject/Object.h"

#include "RemoteControlDMXPerPresetEditorSettings.generated.h"

class URemoteControlPreset;

/** UObject to hold entities list settings data */
UCLASS()
class URemoteControlDMXPerPresetEditorSettings
	: public UObject
{
	GENERATED_BODY()

public:
	/** Returns the remote control DMX editor settings for the specified preset */
	static URemoteControlDMXPerPresetEditorSettings* GetOrCreatePerPresetEditorSettings(URemoteControlPreset* Preset);

	UPROPERTY()
	FRCPanelExposedEntitiesListSettingsData ExposedEntitiesListSettings;
};
