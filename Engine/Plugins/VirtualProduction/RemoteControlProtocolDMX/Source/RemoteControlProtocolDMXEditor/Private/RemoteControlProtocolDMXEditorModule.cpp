// Copyright Epic Games, Inc. All Rights Reserved.

#include "Library/RemoteControlDMXLibraryBuilder.h"
#include "Modules/ModuleManager.h"
#include "RemoteControlDMXAutoBindHandler.h"
#include "RemoteControlDMXExposedEntitiesGroupPatchWidgetFactory.h"
#include "RemoteControlDMXExposedEntitiesListSettings.h"
#include "RemoteControlDMXExposedEntitiesPanelExtender.h"
#include "RemoteControlDMXExposedEntityPatchWidgetFactory.h"
#include "RemoteControlProtocolDMX.h"
#include "RemoteControlProtocolDMXSettings.h"

/**
 * Remote control protocol DMX editor that allows have editor functionality for the protocol
 */
class FRemoteControlProtocolDMXEditorModule : public IModuleInterface
{
	using FRemoteControlDMXExposedEntitiesPanelExtender = UE::RemoteControl::DMX::FRemoteControlDMXExposedEntitiesPanelExtender;

public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
};

void FRemoteControlProtocolDMXEditorModule::StartupModule()
{
	using namespace UE::RemoteControl::DMX;

	FRemoteControlDMXLibraryBuilder::Register();
	FRemoteControlDMXAutoBindHandler::Register();
	FRemoteControlDMXExposedEntitiesListSettings::Register();
	FRemoteControlDMXExposedEntitiesPanelExtender::Register();
	FRemoteControlDMXExposedEntitiesGroupPatchWidgetFactory::Register();
	FRemoteControlDMXExposedEntityPatchWidgetFactory::Register();
}

void FRemoteControlProtocolDMXEditorModule::ShutdownModule()
{
}


IMPLEMENT_MODULE(FRemoteControlProtocolDMXEditorModule, RemoteControlProtocolDMXEditor);
