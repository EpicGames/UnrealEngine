// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRemoteControlDMXExposedEntitiesGroupPatch.h"

#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/RemoteControlDMXLibraryProxy.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SRemoteControlDMXExposedEntitiesGroupPatch"

namespace UE::RemoteControl::DMX
{
	void SRemoteControlDMXExposedEntitiesGroupPatch::Construct(
		const FArguments& InArgs, 
		const TWeakObjectPtr<URemoteControlPreset>& InWeakPreset,
		const TArray<TSharedRef<FRemoteControlProperty>>& InChildProperties)
	{
		WeakPreset = InWeakPreset;
		if (!InWeakPreset.IsValid() || InChildProperties.IsEmpty())
		{
			return;
		}
		URemoteControlPreset* Preset = WeakPreset.Get();
		check(Preset);

		// Get entities in this group
		for (const TSharedRef<FRemoteControlProperty>& Property : InChildProperties)
		{
			for (const FRemoteControlProtocolBinding& Binding : Property->ProtocolBindings)
			{
				const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> Entity = Binding.GetRemoteControlProtocolEntityPtr();
				if (!Entity.IsValid() || !Entity->IsValid())
				{
					continue;
				}

				FRemoteControlDMXProtocolEntity* DMXEntity = Entity->Cast<FRemoteControlDMXProtocolEntity>();
				if (!DMXEntity)
				{
					continue;
				}

				Entities.Add(Entity.ToSharedRef());
			}
		}

		RequestRefresh();

		URemoteControlDMXLibraryProxy::GetOnPostPropertyPatchesChanged().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::RequestRefresh);
		UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnFixtureTypeChanged);
		UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnFixturePatchChanged);
		Preset->OnEntityExposed().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnEntityExposedOrUnexposed);
		Preset->OnEntityUnexposed().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnEntityExposedOrUnexposed);
		Preset->OnEntityRebind().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnEntityRebind);
		Preset->OnEntitiesUpdated().AddSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnEntitiesUpdated);
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::Refresh()
	{
		RefreshTimerHandle.Invalidate();

		// Only show the patch when grouped by owner
		const URemoteControlDMXUserData* DMXUserData = GetDMXUserData();
		if (!DMXUserData || DMXUserData->GetPatchGroupMode() != ERemoteControlDMXPatchGroupMode::GroupByOwner)
		{
			ChildSlot
				[
					SNullWidget::NullWidget
				];

			return;
		}

		// Update the fixture patches that can be used as primary for this patch
		WeakFixturePatches.Reset();

		const UDMXEntityFixturePatch* FixturePatch = GetFixturePatch();
		const UDMXEntityFixtureType* FixtureType = FixturePatch ? FixturePatch->GetFixtureType() : nullptr;
		if (FixturePatch && FixtureType)
		{
			const TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> AllEntities = FRemoteControlDMXProtocolEntity::GetAllDMXProtocolEntitiesInPreset(WeakPreset.Get());
			for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : AllEntities)
			{
				const FRemoteControlDMXProtocolEntity* DMXEntity = Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
				if (!DMXEntity ||
					!DMXEntity->ExtraSetting.bIsPrimaryPatch)
				{
					continue;
				}

				UDMXEntityFixturePatch* OtherFixturePatch = DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch();
				if (OtherFixturePatch &&
					OtherFixturePatch != FixturePatch &&
					OtherFixturePatch->GetFixtureType() == FixtureType)
				{
					WeakFixturePatches.AddUnique(OtherFixturePatch);
				}
			}
		}

		if (Entities.IsEmpty())
		{
			ChildSlot
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GroupCannotBePatchedInfo", "Not bound to DMX"))
			];
		}
		else
		{
			ChildSlot
			[
				SNew(SComboButton)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
						{
							if (const UDMXEntityFixturePatch* FixturePatch = GetFixturePatch())
							{
								const FText FixturePatchNameText = FText::FromString(FixturePatch->Name);
								if (IsPrimaryFixturePatch())
								{
									return FixturePatchNameText;
								}
								else
								{
									return FText::Format(LOCTEXT("SecondaryFixturePatchText", "Same as {0}"), FixturePatchNameText);
								}
							}
							else
							{
								return LOCTEXT("NotPatchedInfo", "Not Patched");
							}
						})
				]
				.MenuContent()
				[
					CreateMenu()
				]
			];
		}
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::RequestRefresh()
	{
		if (!RefreshTimerHandle.IsValid())
		{
			RefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::Refresh));
		}
	}

	TSharedRef<SWidget> SRemoteControlDMXExposedEntitiesGroupPatch::CreateMenu()
	{
		if (!ensureMsgf(!Entities.IsEmpty(), TEXT("Expected valid entities to create fixture patch combo box but got none")))
		{
			return SNullWidget::NullWidget;
		}

		constexpr bool bShouldCloseWindowAfterSelection = true;
		const TSharedPtr<FUICommandList> NullCommandList = nullptr;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterSelection, NullCommandList);
		
		MenuBuilder.SetSearchable(true);
		MenuBuilder.AddSearchWidget();

		const FName NoExtensionHook = NAME_None;
		const FText NoLabel = FText::GetEmpty();
		const FText NoTooltip = FText::GetEmpty();

		// Actions Section
		MenuBuilder.BeginSection("Actions", LOCTEXT("ActionsSectionLabel", "Actions"));
		{
			if (GetFixturePatch())
			{
				// Clear Patch
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ClearPatchLabel", "Clear"),
					LOCTEXT("ClearPatchTooltip", "Clears the fixture patch. The properties will not receive DMX when the patch is cleared."),
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::ClearPatch)
					)
				);
			}
			
			if(!GetFixturePatch() || !IsPrimaryFixturePatch())
			{
				// Generate Patch
				MenuBuilder.AddMenuEntry(
					LOCTEXT("GeneratePatchLabel", "Generate Patch"),
					LOCTEXT("GeneratePatchTooltip", "Generates a new fixture patch for this owner."),
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::GeneratePatch)
					)
				);
			}
		}
		MenuBuilder.EndSection();

		// Fixture Patch section
		if (!WeakFixturePatches.IsEmpty())
		{
			MenuBuilder.BeginSection("FixturePatch", LOCTEXT("FixturePatchSectionLabel", "Patch same as.."));
			{
				for (const TWeakObjectPtr<UDMXEntityFixturePatch>& WeakFixturePatch : WeakFixturePatches)
				{
					UDMXEntityFixturePatch* FixturePatch = WeakFixturePatch.Get();
					if (!FixturePatch)
					{
						continue;
					}

					MenuBuilder.AddMenuEntry
					(
						FText::FromString(FixturePatch->Name),
						NoTooltip,
						FSlateIcon(),
						FUIAction
						(
							FExecuteAction::CreateSP(this, &SRemoteControlDMXExposedEntitiesGroupPatch::OnFixturePatchSelected, WeakFixturePatch)
						),
						NoExtensionHook,
						EUserInterfaceActionType::Button
					);
				}
			}

			MenuBuilder.EndSection();
		}

		return MenuBuilder.MakeWidget();
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::ClearPatch()
	{
		for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Entities)
		{
			if (FRemoteControlDMXProtocolEntity* DMXEntity = Entity->Cast<FRemoteControlDMXProtocolEntity>())
			{
				DMXEntity->ExtraSetting.bIsPrimaryPatch = true;
				DMXEntity->ExtraSetting.FixturePatchReference = nullptr;
				DMXEntity->ExtraSetting.bRequestClearPatch = true;

				DMXEntity->Invalidate();
			}
		}
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::GeneratePatch()
	{
		for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Entities)
		{
			if (FRemoteControlDMXProtocolEntity* DMXEntity = Entity->Cast<FRemoteControlDMXProtocolEntity>())
			{
				DMXEntity->ExtraSetting.bIsPrimaryPatch = true;
				DMXEntity->ExtraSetting.bRequestClearPatch = false;

				DMXEntity->Invalidate();
			}
		}
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::SetAutoPatchEnabled(bool bEnabled)
	{
		if (URemoteControlDMXUserData* DMXUserData = GetDMXUserData())
		{
			DMXUserData->SetAutoPatchEnabled(bEnabled);
		}
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnFixturePatchSelected(TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch)
	{
		if (!ensureMsgf(WeakFixturePatch.IsValid(), TEXT("Trying to set fixture patch for remote control owner, but fixture patch is no longer valid.")))
		{
			return;
		}

		for (const TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>& Entity : Entities)
		{
			if (FRemoteControlDMXProtocolEntity* DMXEntity = Entity->Cast<FRemoteControlDMXProtocolEntity>())
			{
				if (DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() != WeakFixturePatch.Get())
				{
					DMXEntity->ExtraSetting.FixturePatchReference = WeakFixturePatch.Get();
					DMXEntity->ExtraSetting.bIsPrimaryPatch = false;
					DMXEntity->ExtraSetting.bRequestClearPatch = false;

					DMXEntity->Invalidate();
				}
			}
		}
	}

	bool SRemoteControlDMXExposedEntitiesGroupPatch::IsPrimaryFixturePatch() const
	{
		if (Entities.IsEmpty() || !Entities[0]->IsValid())
		{
			return false;
		}

		const FRemoteControlDMXProtocolEntity* DMXEntity = Entities[0]->Cast<FRemoteControlDMXProtocolEntity>();
		return DMXEntity ? DMXEntity->ExtraSetting.bIsPrimaryPatch : false;
	}

	UDMXEntityFixturePatch* SRemoteControlDMXExposedEntitiesGroupPatch::GetFixturePatch() const
	{			
		if (Entities.IsEmpty() || !Entities[0]->IsValid())
		{
			return nullptr;
		}

		const FRemoteControlDMXProtocolEntity* DMXEntity = Entities[0]->Cast<FRemoteControlDMXProtocolEntity>();
		return DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
	}

	URemoteControlDMXUserData* SRemoteControlDMXExposedEntitiesGroupPatch::GetDMXUserData() const
	{
		if (WeakPreset.IsValid())
		{
			return nullptr;
		}

		const TObjectPtr<UObject>* DMXUserDataObjectPtr = Algo::FindByPredicate(WeakPreset.Get()->UserData, [](const TObjectPtr<UObject>& Object)
			{
				return Object && Object->GetClass() == URemoteControlDMXUserData::StaticClass();
			});

		return DMXUserDataObjectPtr ? CastChecked<URemoteControlDMXUserData>(*DMXUserDataObjectPtr) : nullptr;
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnPostLoadRemoteControlPreset(URemoteControlPreset* Preset)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnEntityExposedOrUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnEntityRebind(const FGuid& EntityId)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntitiesGroupPatch::OnEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities)
	{
		RequestRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
