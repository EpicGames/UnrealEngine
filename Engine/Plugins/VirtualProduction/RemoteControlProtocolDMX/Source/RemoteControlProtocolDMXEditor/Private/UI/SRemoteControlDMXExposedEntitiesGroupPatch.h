// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "Widgets/SCompoundWidget.h"

class STableViewBase;
class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class URemoteControlPreset;
class URemoteControlDMXUserData;
enum class ERemoteControlDMXPatchGroupMode : uint8;
struct FRemoteControlProperty;
struct FRemoteControlProtocolEntity;
template<typename T> class TStructOnScope;

namespace UE::RemoteControl::DMX
{
	class FRemoteControlDMXFieldGroupColumnPatchItem;

	class SRemoteControlDMXExposedEntitiesGroupPatch : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRemoteControlDMXExposedEntitiesGroupPatch)
			{}

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TWeakObjectPtr<URemoteControlPreset>& InPreset, const TArray<TSharedRef<FRemoteControlProperty>>& InChildProperties);

	private:
		/** Refreshes the widget */
		void Refresh();

		/** Refreshes the widget on the next tick */
		void RequestRefresh();

		/** Creates the menu */
		TSharedRef<SWidget> CreateMenu();

		/** Clears the patch for all entities */
		void ClearPatch();

		/** Generates a patch for all entities */
		void GeneratePatch(); 

		/** Enables or disables auto patching for the entities in this group */
		void SetAutoPatchEnabled(bool bEnabled);

		/** Called when a fixture patch was selected */
		void OnFixturePatchSelected(TWeakObjectPtr<UDMXEntityFixturePatch> WeakFixturePatch);

		/** Returns true if a primary fixture patch is selected */
		bool IsPrimaryFixturePatch() const;

		/** Returns the selected fixture patch or nullptr if no patch is selected */
		UDMXEntityFixturePatch* GetFixturePatch() const;
		
		/** Returns the remote control DMX user data */
		URemoteControlDMXUserData* GetDMXUserData() const;

		/** Called when a fixture patch changed */
		void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

		/** Called when a fixture type changed */
		void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

		/** Called when the remote control preset was fully loaded */
		void OnPostLoadRemoteControlPreset(URemoteControlPreset* Preset);

		/** Called when an entity was exposed or unexposed */
		void OnEntityExposedOrUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId);

		/** Called when an entity was rebound */
		void OnEntityRebind(const FGuid& EntityId);

		/** Called when entities changed */
		void OnEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities);

		/** The entities that are affected by the patch */
		TArray<TSharedRef<TStructOnScope<FRemoteControlProtocolEntity>>> Entities;

		/** The preset for which this widget is displayed */
		TWeakObjectPtr<URemoteControlPreset> WeakPreset;

		/** The fixture patches in this preset */
		TArray<TWeakObjectPtr<UDMXEntityFixturePatch>> WeakFixturePatches;

		/** Timer handle for the request refresh method */
		FTimerHandle RefreshTimerHandle;
	};
}
