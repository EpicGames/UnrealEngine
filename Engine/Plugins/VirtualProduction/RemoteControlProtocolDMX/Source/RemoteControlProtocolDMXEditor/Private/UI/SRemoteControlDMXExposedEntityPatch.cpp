// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRemoteControlDMXExposedEntityPatch.h"

#include "Editor.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/RemoteControlDMXLibraryProxy.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolDMX.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"
#include <limits>

#define LOCTEXT_NAMESPACE "SRemoteControlDMXExposedEntityPatch"

namespace UE::RemoteControl::DMX
{
	void SRemoteControlDMXExposedEntityPatch::Construct(const FArguments& InArgs, const TWeakObjectPtr<URemoteControlPreset>& InPreset, const TSharedRef<FRemoteControlProperty>& InProperty)
	{
		WeakPreset = InPreset;
		if (!WeakPreset.IsValid())
		{
			return;
		}
		URemoteControlPreset* Preset = WeakPreset.Get();
		check(Preset);

		Property = InProperty;

		URemoteControlDMXLibraryProxy::GetOnPostPropertyPatchesChanged().AddSP(this, &SRemoteControlDMXExposedEntityPatch::RequestRefresh);
		Preset->OnEntityExposed().AddSP(this, &SRemoteControlDMXExposedEntityPatch::OnEntityExposedOrUnexposed);
		Preset->OnEntityUnexposed().AddSP(this, &SRemoteControlDMXExposedEntityPatch::OnEntityExposedOrUnexposed);
		Preset->OnEntityRebind().AddSP(this, &SRemoteControlDMXExposedEntityPatch::OnEntityRebind);
		Preset->OnEntitiesUpdated().AddSP(this, &SRemoteControlDMXExposedEntityPatch::OnEntitiesUpdated);

		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntityPatch::RequestRefresh()
	{
		if (!RefreshTimerHandle.IsValid())
		{
			RefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SRemoteControlDMXExposedEntityPatch::Refresh));
		}
	}

	void SRemoteControlDMXExposedEntityPatch::Refresh()
	{
		RefreshTimerHandle.Invalidate();

		TPair<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>, int64> MinEntityAbsoluteAddressPair = { nullptr, std::numeric_limits<int64>::max() };
		for (const FRemoteControlProtocolBinding& Binding : Property->ProtocolBindings)
		{
			const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> Entity = Binding.GetRemoteControlProtocolEntityPtr();
			const FRemoteControlDMXProtocolEntity* DMXEntity = Entity.IsValid() && Entity->IsValid() ? Entity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
			const UDMXEntityFixturePatch* FixturePatch = DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
			const FDMXFixtureMode* ActiveModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;

			if (FixturePatch && 
				ActiveModePtr &&
				ActiveModePtr->Functions.IsValidIndex(DMXEntity->ExtraSetting.FunctionIndex))
			{
				const FDMXFixtureFunction& Function = ActiveModePtr->Functions[DMXEntity->ExtraSetting.FunctionIndex];

				const int32 FunctionStartingChannel = FixturePatch->GetStartingChannel() + Function.Channel - 1;
				const int64 AbsoluteAddress = (int64)FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE + FunctionStartingChannel;
				const int64 PreviousMinAbsoluteAddress = MinEntityAbsoluteAddressPair.Value;
				
				if (PreviousMinAbsoluteAddress > AbsoluteAddress)
				{
					MinEntityAbsoluteAddressPair = TPair<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>, int64>(Entity, AbsoluteAddress);
				}
			}
		}

		if (!MinEntityAbsoluteAddressPair.Key.IsValid())
		{
			return;
		}

		MinEntity = MinEntityAbsoluteAddressPair.Key;

		ChildSlot
		[
			SNew(SEditableTextBox)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text(this, &SRemoteControlDMXExposedEntityPatch::GetUniverseChannelText)
			.ToolTipText(LOCTEXT("UniverseChannelTooltip", "The universe and channel in the form of Universe.Channe. E.g. '1.1' is universe 1 channel 1."))
			.IsReadOnly(this, &SRemoteControlDMXExposedEntityPatch::IsReadOnly)
			.OnTextCommitted(this, &SRemoteControlDMXExposedEntityPatch::OnUniverseChannelTextCommitted)
		];
	}

	FText SRemoteControlDMXExposedEntityPatch::GetUniverseChannelText() const
	{
		const FRemoteControlDMXProtocolEntity* DMXEntity = MinEntity.IsValid() && MinEntity->IsValid() ? MinEntity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		const UDMXEntityFixturePatch* FixturePatch = DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
		const FDMXFixtureMode* ModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;

		if (ModePtr && ModePtr->Functions.IsValidIndex(DMXEntity->ExtraSetting.FunctionIndex))
		{
			const FDMXFixtureFunction& Function = ModePtr->Functions[DMXEntity->ExtraSetting.FunctionIndex];

			const int32 FunctionStartingChannel = FixturePatch->GetStartingChannel() + Function.Channel - 1;
			const FString UniverseChannelString = FString::Printf(TEXT("%i.%i"), FixturePatch->GetUniverseID(), FunctionStartingChannel);
			return FText::FromString(UniverseChannelString);
		}
		else
		{
			return LOCTEXT("InvalidEntity", "Invalid Patch");
		}
	}

	void SRemoteControlDMXExposedEntityPatch::OnUniverseChannelTextCommitted(const FText& InUniverseChannelText, ETextCommit::Type InCommitType)
	{
		const FRemoteControlDMXProtocolEntity* DMXEntity = MinEntity.IsValid() && MinEntity->IsValid() ? MinEntity->Cast<FRemoteControlDMXProtocolEntity>() : nullptr;
		UDMXEntityFixturePatch* FixturePatch = DMXEntity ? DMXEntity->ExtraSetting.FixturePatchReference.GetFixturePatch() : nullptr;
		if (!FixturePatch)
		{
			return;
		}

		TArray<FString> UniverseChannelStrings;
		InUniverseChannelText.ToString().ParseIntoArray(UniverseChannelStrings, TEXT("."));

		int32 Universe;
		int32 Channel;
		if (UniverseChannelStrings.Num() == 2 &&
			LexTryParseString(Universe, *UniverseChannelStrings[0]) &&
			LexTryParseString(Channel, *UniverseChannelStrings[1]))
		{
			const FScopedTransaction ReassignFixturePatchTransaction(LOCTEXT("ReassignFixturePatchTransaction", "Reassign Remote Control DMX Patch"));

			FixturePatch->PreEditChange(nullptr);

			FixturePatch->SetUniverseID(Universe);
			FixturePatch->SetStartingChannel(Channel);

			FixturePatch->PostEditChange();
		}
	}

	bool SRemoteControlDMXExposedEntityPatch::IsReadOnly() const
	{
		const URemoteControlDMXUserData* DMXUserData = URemoteControlDMXUserData::GetOrCreateDMXUserData(WeakPreset.Get());
		return DMXUserData ? DMXUserData->IsAutoPatch() : true;
	}

	void SRemoteControlDMXExposedEntityPatch::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntityPatch::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntityPatch::OnPostLoadRemoteControlPreset(URemoteControlPreset* Preset)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntityPatch::OnEntityExposedOrUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntityPatch::OnEntityRebind(const FGuid& EntityId)
	{
		RequestRefresh();
	}

	void SRemoteControlDMXExposedEntityPatch::OnEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities)
	{
		RequestRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
