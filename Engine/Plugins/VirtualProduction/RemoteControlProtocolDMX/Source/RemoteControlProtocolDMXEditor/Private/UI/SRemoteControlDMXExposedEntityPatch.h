// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "Widgets/SCompoundWidget.h"

class UDMXEntityFixturePatch;
class UDMXEntityFixtureType;
class URemoteControlDMXUserData;
class URemoteControlPreset;
enum class ERCFieldGroupType : uint8;
struct FRemoteControlProperty;
struct FRemoteControlProtocolEntity;
template<typename T> class TStructOnScope;

namespace UE::RemoteControl::DMX
{
	class SRemoteControlDMXExposedEntityPatch : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRemoteControlDMXExposedEntityPatch)
			{}

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TWeakObjectPtr<URemoteControlPreset>& InPreset, const TSharedRef<FRemoteControlProperty>& InProperty);

	private:
		/** Refreshes the widget on the next tick */
		void RequestRefresh();

		/** Refreshes the widget */
		void Refresh();

		/** Returns the current universe channel text */
		FText GetUniverseChannelText() const;

		/** Called when universe channel text was committed */
		void OnUniverseChannelTextCommitted(const FText& InUniverseChannelText, ETextCommit::Type InCommitType);

		/** Returns true if the text block of this widget is read only */
		bool IsReadOnly() const;

		/** Called when a fixture patch changed */
		void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

		/** Called when a fixture type changed */
		void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

		/** Called when the remote control preset was fully loaded */
		void OnPostLoadRemoteControlPreset(URemoteControlPreset* Preset);

		/** Called when an entity was exposed or unexposed */
		void OnEntityExposedOrUnexposed(URemoteControlPreset* Preset, const FGuid& EntityId);

		/** Called when an entity was rebound */
		void OnEntityRebind(const FGuid& EntityId);

		/** Called when entities changed */
		void OnEntitiesUpdated(URemoteControlPreset* Preset, const TSet<FGuid>& ModifiedEntities);

		/** The displayed property */
		TSharedPtr<FRemoteControlProperty> Property;
		
		/** The first binding given the DMX universe and channel */
		TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> MinEntity;

		/** The preset for which this widget is painted */
		TWeakObjectPtr<URemoteControlPreset> WeakPreset;

		/** Timer handle for request refresh */
		FTimerHandle RefreshTimerHandle;
	};
}
