// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRemoteControlDMXPresetUserData.h"

#include "AssetToolsModule.h"
#include "Customizations/RemoteControlProtocolDMXPresetUserDataDetails.h"
#include "DetailsViewArgs.h"
#include "DMXEditorModule.h"
#include "DMXEditorStyle.h"
#include "Factories/DMXLibraryFactory.h"
#include "IDetailsView.h"
#include "IRCProtocolBindingList.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#include "Library/DMXLibrary.h"
#include "Library/DMXLibrary.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RemoteControlDMXUserData.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "UObject/GarbageCollection.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SRemoteControlDMXPresetUserData"

namespace UE::RemoteControl::DMX
{
	void SRemoteControlDMXPresetUserData::Construct(const FArguments& InArgs, URemoteControlDMXUserData* InDMXUserData)
	{
		if (!ensureMsgf(InDMXUserData, TEXT("%hs: Invalid Remote Control DMX User Data provided, cannot draw widget."), __FUNCTION__))
		{
			return;
		}
		DMXUserData = InDMXUserData;
		
		// Create a details view for the user data to present the DMX library
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bShowOptions = false;

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		
		DetailsView->RegisterInstancedCustomPropertyLayout(URemoteControlDMXUserData::StaticClass(), 
			FOnGetDetailCustomizationInstance::CreateStatic(&FRemoteControlProtocolDMXPresetUserDataDetails::MakeInstance));
		
		DetailsView->SetObject(InDMXUserData);

		ChildSlot
			[
				SNew(SHorizontalBox)

				// Auto Patch option
				+ SHorizontalBox::Slot()
				.Padding(FMargin(8.f, 0.f))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SRemoteControlDMXPresetUserData::GetAutoPatchCheckState)
					.OnCheckStateChanged(this, &SRemoteControlDMXPresetUserData::OnAutoPatchCheckStateChanged)
					.ToolTipText(LOCTEXT("AutoPatchCheckBoxTooltip", "Enables auto assign patches. Note, in auto assign mode patches are created depening on the sort order.."))
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(FMargin(8.f, 0.f))
						[
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Text(LOCTEXT("AutoPatchCheckBoxLabel", "Auto Patch in List Order"))
						]
					]
				]
					
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSeparator)
					.Orientation(Orient_Vertical)
				]

				// Auto assign from universe
				+ SHorizontalBox::Slot()
				.Padding(FMargin(8.f, 0.f))
				.AutoWidth()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.Padding(FMargin(4.f, 0.f))
					.Visibility(this, &SRemoteControlDMXPresetUserData::GetAutoAssignFromUniverseVisibility)
					[
						SNew(SHorizontalBox)

						// Label
						+ SHorizontalBox::Slot()
						.Padding(FMargin(4.f, 0.f))
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AutoAssignFromUniverseLabel", "Patch from Universe"))
						]

						// Editable text block
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(4.f, 0.f))
						.AutoWidth()
						[
							SAssignNew(AutoAssignFromUniverseEditableTextBox, SEditableTextBox)
							.MinDesiredWidth(40.f)
							.TextFlowDirection(ETextFlowDirection::RightToLeft)
							.Text(this, &SRemoteControlDMXPresetUserData::GetAutoAssignFromUniverseText)
							.OnTextCommitted(this, &SRemoteControlDMXPresetUserData::OnAutoAssignFromUniverseTextCommitted)
						]
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSeparator)
					.Orientation(Orient_Vertical)
				]

				// Actions menu
				+SHorizontalBox::Slot()
				.Padding(FMargin(8.f, 0.f))
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				[
					GenerateActionsMenu()
				]

				// DMXLibrary
				+ SHorizontalBox::Slot()
				.Padding(FMargin(8.f, 0.f))
				.VAlign(VAlign_Center)	
				.AutoWidth()
				[
					DetailsView
				]
			];
	}

	void SRemoteControlDMXPresetUserData::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(DMXUserData);
	}
	
	FString SRemoteControlDMXPresetUserData::GetReferencerName() const
	{
		return TEXT("SRemoteControlDMXPresetUserData");
	}
	
	TSharedRef<SWidget> SRemoteControlDMXPresetUserData::GenerateActionsMenu()
	{
		constexpr const TCHAR* MenuName = TEXT("RemoteControlProtocolDMXMenu");

		UToolMenus* ToolMenus = UToolMenus::Get();
		check(ToolMenus);

		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			const FName NoParentToolbar = NAME_None;
			ToolMenus->RegisterMenu(MenuName, NoParentToolbar, EMultiBoxType::SlimHorizontalToolBar);
		}
		
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(MenuName);
		check(Menu);

		const FText NoLabel;
		FToolMenuSection& ActionsSection = Menu->AddSection("Actions", NoLabel);

		FToolMenuEntry ExportMVREntry = FToolMenuEntry::InitToolBarButton(
			"ExportMVR",
			FUIAction(FExecuteAction::CreateSP(this, &SRemoteControlDMXPresetUserData::OnExportAsMVRClicked)),
			LOCTEXT("ExportAsMVRLabel", "Export as MVR"),
			LOCTEXT("ExportAsMVRTooltip", "Exports the Remote Control DMX Library as MVR file"),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.DMXLibraryToolbar.Export"));

		ActionsSection.AddEntry(ExportMVREntry);

		FToolMenuEntry GenerateDMXLibraryEntry = FToolMenuEntry::InitToolBarButton(
			"GenerateDMXLibrary",
			FUIAction(FExecuteAction::CreateSP(this, &SRemoteControlDMXPresetUserData::OnCreateDMXLibraryClicked)),
			LOCTEXT("GenerateDMXLibraryLabel", "Create DMX Library"),
			LOCTEXT("GenerateDMXLibraryTooltip", "Creates a new DMX Library asset from this Remote Control Preset"),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "ClassIcon.DMXLibrary"));

		ActionsSection.AddEntry(GenerateDMXLibraryEntry);

		return ToolMenus->GenerateWidget(Menu);
	}

	void SRemoteControlDMXPresetUserData::OnExportAsMVRClicked()
	{
		UDMXLibrary* DMXLibrary = DMXUserData ? DMXUserData->GetDMXLibrary() : nullptr;
		if (DMXLibrary)
		{
			URemoteControlPreset* RemoteControlPreset = Cast<URemoteControlPreset>(DMXUserData->GetOuter());
			const FString DesiredFileName = RemoteControlPreset ? RemoteControlPreset->GetName() : FString();

			const FDMXEditorModule& DMXEditorModule = FModuleManager::GetModuleChecked<FDMXEditorModule>("DMXEditor");
			DMXEditorModule.ExportDMXLibraryAsMVRFile(DMXLibrary, DesiredFileName);
		}
	}

	void SRemoteControlDMXPresetUserData::OnCreateDMXLibraryClicked()
	{
		if (DMXUserData)
		{
			UDMXLibraryFactory* DMXLibraryFactory = NewObject<UDMXLibraryFactory>();

			// Ensure this object is not GC for the duration of CreateAssetWithDialog
			FGCScopeGuard GCGuard;

			FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
			UObject* NewDMXLibraryObject = AssetToolsModule.Get().CreateAssetWithDialog(DMXLibraryFactory->GetSupportedClass(), DMXLibraryFactory);

			// Set the DMX Library 
			if (UDMXLibrary* NewDMXLibrary = Cast<UDMXLibrary>(NewDMXLibraryObject))
			{
				DMXUserData->SetDMXLibrary(NewDMXLibrary);
			}
		}
	}

	ECheckBoxState SRemoteControlDMXPresetUserData::GetAutoPatchCheckState() const
	{
		return DMXUserData && DMXUserData->IsAutoPatch() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SRemoteControlDMXPresetUserData::OnAutoPatchCheckStateChanged(ECheckBoxState NewCheckState)
	{
		const bool bAutoPatch = NewCheckState == ECheckBoxState::Checked;
		DMXUserData->SetAutoPatchEnabled(bAutoPatch);

		const IRemoteControlProtocolWidgetsModule& RCWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
		const TSharedPtr<IRCProtocolBindingList> BindingList = RCWidgetsModule.GetProtocolBindingList();
		if (BindingList.IsValid())
		{
			BindingList->Refresh();
		}
	}

	EVisibility SRemoteControlDMXPresetUserData::GetAutoAssignFromUniverseVisibility() const
	{
		return DMXUserData && DMXUserData->IsAutoPatch() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText SRemoteControlDMXPresetUserData::GetAutoAssignFromUniverseText() const
	{
		if (DMXUserData)
		{
			const int32 AutoAssignFromUniverse = DMXUserData->GetAutoAssignFromUniverse();
			const FString AutoAssignFromUniverseString = FString::FromInt(AutoAssignFromUniverse);
			return FText::FromString(AutoAssignFromUniverseString);
		}

		return FText::GetEmpty();
	}

	void SRemoteControlDMXPresetUserData::OnAutoAssignFromUniverseTextCommitted(const FText& InAutoAssignFromUniverseText, ETextCommit::Type InCommitType)
	{
		const FScopedTransaction ReassignFixturePatchTransaction(LOCTEXT("ReassignFixturePatchesTransaction", "Set Patch To DMX Universe"));

		int32 AutoAssignFromUniverse = 0;
		if (DMXUserData &&
			LexTryParseString(AutoAssignFromUniverse, *InAutoAssignFromUniverseText.ToString()) &&
			AutoAssignFromUniverse > 0)
		{
			DMXUserData->PreEditChange(nullptr);
			DMXUserData->SetAutoAssignFromUniverse(AutoAssignFromUniverse);
			DMXUserData->PostEditChange();
		}
		else if (DMXUserData)
		{
			const FText AutoAssignFromUniverseText = FText::FromString(FString::FromInt(DMXUserData->GetAutoAssignFromUniverse()));
			AutoAssignFromUniverseEditableTextBox->SetText(AutoAssignFromUniverseText);
		}
	}
}

#undef LOCTEXT_NAMESPACE
