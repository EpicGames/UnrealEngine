// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
class URemoteControlDMXUserData;
enum class ECheckBoxState : uint8;

namespace UE::RemoteControl::DMX
{
	class SRemoteControlDMXPresetUserData : public SCompoundWidget, public FGCObject
	{
	public:
		SLATE_BEGIN_ARGS(SRemoteControlDMXPresetUserData) 
			{}

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, URemoteControlDMXUserData* InDMXUserData);

	protected:
		// FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override;
		// End FGCObject interface

	private:
		/** Generates an Actions menu (e.g Export as MVR) */
		TSharedRef<SWidget> GenerateActionsMenu();

		/** Called when the export as MVR button was clicked */
		void OnExportAsMVRClicked();

		/** Called when the Create DMX Library button was clicked */
		void OnCreateDMXLibraryClicked();

		/** Returns true if the auto patch option is checked */
		ECheckBoxState GetAutoPatchCheckState() const;

		/** Called when the auto patch check state changed */
		void OnAutoPatchCheckStateChanged(ECheckBoxState NewCheckState);
		
		/** Returns the visibility of the auto assign from universe option */
		EVisibility GetAutoAssignFromUniverseVisibility() const;

		/** Returns the auto assign from universe as text */
		FText GetAutoAssignFromUniverseText() const;

		/** Called when the auto assign from universe text was committed */
		void OnAutoAssignFromUniverseTextCommitted(const FText& InAutoAssignFromUniverseText, ETextCommit::Type InCommitType);
		
		/** Text box to edit the auto assign from universe */
		TSharedPtr<SEditableTextBox> AutoAssignFromUniverseEditableTextBox;

		/** The preset this widget is editing */
		TObjectPtr<URemoteControlDMXUserData> DMXUserData;
	};
}

