// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlProtocolDMXEditor : ModuleRules
{
	public RemoteControlProtocolDMXEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "RCPDMXEditor";

		PublicDependencyModuleNames.AddRange(
			new string[] {}
		);
		
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DMXEditor",
				"DMXProtocol",
				"DMXRuntime",
				"DMXProtocolEditor",
				"Engine",
				"InputCore",
				"PropertyEditor",
				"RemoteControl",
				"RemoteControlProtocol",
				"RemoteControlProtocolDMX",
				"RemoteControlUI",
				"RemoteControlProtocolWidgets",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd"
			}
		);
    }
}
